// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraAnimationSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAnimationSequence)

UCameraAnimationSequence::UCameraAnimationSequence(const FObjectInitializer& ObjectInitializer)
    : UTemplateSequence(ObjectInitializer)
{
	// TODO: warn about saving property values in the spawnable template
}

#if WITH_EDITOR
FText UCameraAnimationSequence::GetDisplayName() const
{
	return UMovieSceneSequence::GetDisplayName();
}
#endif

