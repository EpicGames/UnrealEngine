// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraAnimationSequencePlayer.h"
#include "Camera/CameraTypes.h"
#include "CineCameraComponent.h"
#include "Engine/World.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieScenePropertySystemTypes.inl"
#include "GameFramework/WorldSettings.h"
#include "CameraAnimationSequenceSubsystem.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTracksComponentTypes.h"
#include "MovieSceneSpawnable.h"
#include "TemplateSequence.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAnimationSequencePlayer)

namespace UE
{
namespace MovieScene
{

FIntermediate3DTransform GetCameraStandInTransform(const UObject* Object)
{
	const UCameraAnimationSequenceCameraStandIn* CameraStandIn = CastChecked<const UCameraAnimationSequenceCameraStandIn>(Object);
	FIntermediate3DTransform Result;
	ConvertOperationalProperty(CameraStandIn->GetTransform(), Result);
	return Result;
}

void SetCameraStandInTransform(UObject* Object, const FIntermediate3DTransform& InTransform)
{
	UCameraAnimationSequenceCameraStandIn* CameraStandIn = CastChecked<UCameraAnimationSequenceCameraStandIn>(Object);
	FTransform Result;
	ConvertOperationalProperty(InTransform, Result);
	CameraStandIn->SetTransform(Result);
}

template<typename PropertyTraits, typename MetaDataIndices>
struct TUpdateInitialPropertyValuesImpl;

template<typename PropertyTraits, int ...MetaDataIndices>
struct TUpdateInitialPropertyValuesImpl<PropertyTraits, TIntegerSequence<int, MetaDataIndices...>>
{
	void Update(UMovieSceneEntitySystemLinker* Linker, const TPropertyComponents<PropertyTraits>& PropertyComponents)
	{
		const FBuiltInComponentTypes* const BuiltInComponents = FBuiltInComponentTypes::Get();

		const FPropertyDefinition& PropertyDefinition = BuiltInComponents->PropertyRegistry.GetDefinition(PropertyComponents.CompositeID);

		TGetPropertyValues<PropertyTraits> GetProperties(&PropertyComponents.PropertyTraits, PropertyDefinition.CustomPropertyRegistration);

		FEntityTaskBuilder()
			.Read(BuiltInComponents->BoundObject)
			.ReadOneOf(BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty)
			.ReadAllOf(PropertyComponents.MetaDataComponents)
			.Write(PropertyComponents.InitialValue)
			.FilterAll({ PropertyComponents.PropertyTag })
			.SetDesiredThread(Linker->EntityManager.GetGatherThread())
			.RunInline_PerAllocation(&Linker->EntityManager, GetProperties);
	}
};

template<typename PropertyTraits>
void UpdateInitialPropertyValues(UMovieSceneEntitySystemLinker* Linker, const TPropertyComponents<PropertyTraits>& PropertyComponents)
{
	TUpdateInitialPropertyValuesImpl<PropertyTraits, TMakeIntegerSequence<int, PropertyTraits::MetaDataType::Num>>()
		.Update(Linker, PropertyComponents);
}

}
}

bool UCameraAnimationSequenceCameraStandIn::bRegistered(false);

UCameraAnimationSequenceCameraStandIn::UCameraAnimationSequenceCameraStandIn(const FObjectInitializer& ObjInit) 
	: Super(ObjInit) 
{
}

void UCameraAnimationSequenceCameraStandIn::Initialize(UTemplateSequence* TemplateSequence)
{
	AActor* CameraTemplate = nullptr;
	UMovieScene* MovieScene = TemplateSequence->GetMovieScene();
	const FGuid RootObjectBindingID = TemplateSequence->GetRootObjectBindingID();
	if (MovieScene && RootObjectBindingID.IsValid())
	{
		if (FMovieSceneSpawnable* RootObjectSpawnable = MovieScene->FindSpawnable(RootObjectBindingID))
		{
			CameraTemplate = Cast<AActor>(RootObjectSpawnable->GetObjectTemplate());
		}
	}

	bIsCineCamera = false;
	bool bGotInitialValues = false;

	if (CameraTemplate)
	{
		if (UCineCameraComponent* CineCameraComponent = CameraTemplate->FindComponentByClass<UCineCameraComponent>())
		{
			bIsCineCamera = true;
			bGotInitialValues = true;

			FieldOfView = CineCameraComponent->FieldOfView;
			AspectRatio = CineCameraComponent->AspectRatio;
			PostProcessSettings = CineCameraComponent->PostProcessSettings;
			PostProcessBlendWeight = CineCameraComponent->PostProcessBlendWeight;

			Filmback = CineCameraComponent->Filmback;
			LensSettings = CineCameraComponent->LensSettings;
			FocusSettings = CineCameraComponent->FocusSettings;
			CurrentFocalLength = CineCameraComponent->CurrentFocalLength;
			CurrentAperture = CineCameraComponent->CurrentAperture;
			CurrentFocusDistance = CineCameraComponent->CurrentFocusDistance;

			// Get the world unit to meters scale.
			UWorld const* const World = GetWorld();
			AWorldSettings const* const WorldSettings = World ? World->GetWorldSettings() : nullptr;
			WorldToMeters = WorldSettings ? WorldSettings->WorldToMeters : 100.f;
		}
		else if (UCameraComponent* CameraComponent = CameraTemplate->FindComponentByClass<UCameraComponent>())
		{
			bGotInitialValues = true;

			FieldOfView = CameraComponent->FieldOfView;
			AspectRatio = CameraComponent->AspectRatio;
			PostProcessSettings = CameraComponent->PostProcessSettings;
			PostProcessBlendWeight = CameraComponent->PostProcessBlendWeight;
		}

		// We reset our transform to identity because we want to be able to treat the animated 
		// transform as an additive value in local camera space. As a result, we won't need to 
		// synchronize it with the current view info in Reset below.
		Transform = FTransform::Identity;
	}

	ensureMsgf(
			bGotInitialValues, 
			TEXT("Couldn't initialize sequence camera shake: the given sequence may not be animating a camera!"));
}

void UCameraAnimationSequenceCameraStandIn::Reset(const FMinimalViewInfo& ViewInfo, UMovieSceneEntitySystemLinker* Linker)
{
	// Reset the camera stand-in's properties based on the new "current" (unshaken) values.
	ResetDefaultValues(ViewInfo);

	// Sequencer animates things based on the initial values cached when the sequence started. But here we want
	// to animate things based on the moving current values of the camera... i.e., we want to shake or animate 
	// a constantly moving camera. So every frame, we need to update the initial values that sequencer uses.
	UpdateInitialPropertyValues(Linker);
}

void UCameraAnimationSequenceCameraStandIn::ResetDefaultValues(const FMinimalViewInfo& ViewInfo)
{
	// We reset all the properties to the current view's values because a lot of them, like 
	// FieldOfView, don't have any "zero" value that makes sense. We'll figure out the delta in the
	// update code.
	bConstrainAspectRatio = ViewInfo.bConstrainAspectRatio;
	AspectRatio = ViewInfo.AspectRatio;
	FieldOfView = ViewInfo.FOV;
	PostProcessSettings = ViewInfo.PostProcessSettings;
	PostProcessBlendWeight = ViewInfo.PostProcessBlendWeight;

	// We've set the FieldOfView we have to update the CurrentFocalLength accordingly.
	CurrentFocalLength = (Filmback.SensorWidth / 2.f) / FMath::Tan(FMath::DegreesToRadians(FieldOfView / 2.f));

	RecalcDerivedData();
}

void UCameraAnimationSequenceCameraStandIn::UpdateInitialPropertyValues(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	check(Linker);
	UE::MovieScene::UpdateInitialPropertyValues(Linker, TrackComponents->Float);
	// TODO: also do uint8:1/boolean properties?
}

void UCameraAnimationSequenceCameraStandIn::RecalcDerivedData()
{
	if (bIsCineCamera)
	{
		CurrentFocalLength = FMath::Clamp(CurrentFocalLength, LensSettings.MinFocalLength, LensSettings.MaxFocalLength);
		CurrentAperture = FMath::Clamp(CurrentAperture, LensSettings.MinFStop, LensSettings.MaxFStop);

		float const MinFocusDistInWorldUnits = LensSettings.MinimumFocusDistance * (WorldToMeters / 1000.f);	// convert mm to uu
		FocusSettings.ManualFocusDistance = FMath::Max(FocusSettings.ManualFocusDistance, MinFocusDistInWorldUnits);

		float const HorizontalFieldOfView = (CurrentFocalLength > 0.f)
			? FMath::RadiansToDegrees(2.f * FMath::Atan(Filmback.SensorWidth / (2.f * CurrentFocalLength)))
			: 0.f;
		FieldOfView = HorizontalFieldOfView;
		Filmback.SensorAspectRatio = (Filmback.SensorHeight > 0.f) ? (Filmback.SensorWidth / Filmback.SensorHeight) : 0.f;
		AspectRatio = Filmback.SensorAspectRatio;
	}
}

void UCameraAnimationSequenceCameraStandIn::RegisterCameraStandIn()
{
	using namespace UE::MovieScene;

	if (ensure(!bRegistered))
	{
		FMovieSceneTracksComponentTypes* TracksComponentTypes = FMovieSceneTracksComponentTypes::Get();
		TracksComponentTypes->Accessors.ComponentTransform.Add(UCameraAnimationSequenceCameraStandIn::StaticClass(), TEXT("Transform"), &GetCameraStandInTransform, &SetCameraStandInTransform);

		bRegistered = true;
	}
}

void UCameraAnimationSequenceCameraStandIn::UnregisterCameraStandIn()
{
	using namespace UE::MovieScene;

	if (ensure(bRegistered) && UObjectInitialized())
	{
		FMovieSceneTracksComponentTypes* TracksComponentTypes = FMovieSceneTracksComponentTypes::Get();
		TracksComponentTypes->Accessors.ComponentTransform.RemoveAll(UCameraAnimationSequenceCameraStandIn::StaticClass());

		bRegistered = false;
	}
}

UCameraAnimationSequencePlayer::UCameraAnimationSequencePlayer(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, StartFrame(0)
	, Status(EMovieScenePlayerStatus::Stopped)
{
	PlayPosition.Reset(FFrameTime(0));
}

UCameraAnimationSequencePlayer::~UCameraAnimationSequencePlayer()
{
}

void UCameraAnimationSequencePlayer::BeginDestroy()
{
	RootTemplateInstance.TearDown();

	Super::BeginDestroy();
}

UMovieSceneEntitySystemLinker* UCameraAnimationSequencePlayer::ConstructEntitySystemLinker()
{
	UCameraAnimationSequenceSubsystem* Subsystem = UCameraAnimationSequenceSubsystem::GetCameraAnimationSequenceSubsystem(GetWorld());
	if (ensure(Subsystem))
	{
		UMovieSceneEntitySystemLinker* Linker = Subsystem->GetLinker();
		if (ensure(Linker))
		{
			return Linker;
		}
	}
	UMovieSceneEntitySystemLinker* NewLinker = UCameraAnimationSequenceSubsystem::CreateLinker(GetTransientPackage(), TEXT("StandaloneCameraAnimationLinker"));
	return NewLinker;
}

EMovieScenePlayerStatus::Type UCameraAnimationSequencePlayer::GetPlaybackStatus() const
{
	return Status;
}

void UCameraAnimationSequencePlayer::ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& InSequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	if (BoundObjectOverride)
	{
		OutObjects.Add(BoundObjectOverride);
	}
}

void UCameraAnimationSequencePlayer::SetBoundObjectOverride(UObject* InObject)
{
	BoundObjectOverride = InObject;

	SpawnRegister.SetSpawnedObject(InObject);
}

FFrameTime UCameraAnimationSequencePlayer::GetDuration() const
{
	return DurationFrames;
}

FFrameTime UCameraAnimationSequencePlayer::GetCurrentPosition() const
{
	return PlayPosition.GetCurrentPosition();
}

void UCameraAnimationSequencePlayer::Initialize(UMovieSceneSequence* InSequence, int32 StartOffset, float DurationOverride)
{
	checkf(InSequence, TEXT("Invalid sequence given to player"));
	
	if (Sequence)
	{
		Stop();

		if (UMovieSceneEntitySystemLinker* Linker = RootTemplateInstance.GetEntitySystemLinker())
		{
			Linker->Events.AbandonLinker.RemoveAll(this);
		}
	}

	Sequence = InSequence;

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (ensure(MovieScene))
	{
		const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		const FFrameRate TickResolution = MovieScene->GetTickResolution();
		const EMovieSceneEvaluationType EvaluationType = MovieScene->GetEvaluationType();
		
		PlayPosition.SetTimeBase(DisplayRate, TickResolution, EvaluationType);

		const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();

		const FFrameNumber StartTick = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
		const FFrameTime StartTime = ConvertFrameTime(StartTick, TickResolution, DisplayRate);

		const FFrameNumber EndTick = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);
		const FFrameTime EndTime = ConvertFrameTime(EndTick, TickResolution, DisplayRate);

		// Level sequences round starting time to a frame so let's do the same.
		if (StartOffset > 0)
		{
			StartFrame = StartTime.FloorToFrame() + StartOffset;
		}
		else
		{
			StartFrame = StartTime.FloorToFrame();
		}

		DurationFrames = (EndTime - StartFrame);
		TotalDurationFrames = DurationFrames;
		bDurationRequiresLooping = false;

		if (DurationOverride > 0.f)
		{
			const FFrameTime DurationOverrideFrames = DurationOverride * DisplayRate;
			if (DurationOverrideFrames > DurationFrames)
			{
				TotalDurationFrames = DurationOverrideFrames;
				bDurationRequiresLooping = true;
			}
			else
			{
				DurationFrames = DurationOverrideFrames;
			}
		}
	}
	else
	{
		StartFrame = FFrameNumber(0);
		DurationFrames = FFrameTime(0);
		TotalDurationFrames = DurationFrames;
		bDurationRequiresLooping = false;
	}

	PlayPosition.Reset(StartFrame);
	LoopsPlayed = 0;

	UCameraAnimationSequenceSubsystem* Subsystem = UCameraAnimationSequenceSubsystem::GetCameraAnimationSequenceSubsystem(GetWorld());
	ensureMsgf(Subsystem, TEXT("Unable to locate a valid camera animation sub-system. Camera anim sequences will not play."));
	RootTemplateInstance.Initialize(*Sequence, *this, nullptr);

	if (UMovieSceneEntitySystemLinker* Linker = RootTemplateInstance.GetEntitySystemLinker())
	{
		Linker->Events.AbandonLinker.AddUObject(this, &UCameraAnimationSequencePlayer::OnAbandonLinker);
	}
}

void UCameraAnimationSequencePlayer::OnAbandonLinker(UMovieSceneEntitySystemLinker* InLinker)
{
	// Camera animations and camera shakes playing camera animations can outlive the
	// level in which the linker lives. In this case we just stop.
	RootTemplateInstance.TearDown();
}

bool UCameraAnimationSequencePlayer::IsValid() const
{
	return RootTemplateInstance.IsValid();
}

void UCameraAnimationSequencePlayer::Play(bool bLoop, bool bRandomStartTime)
{
	checkf(Sequence, TEXT("No sequence is set on this player, did you forget to call Initialize?"));
	checkf(RootTemplateInstance.IsValid(), TEXT("No evaluation template was created, did you forget to call Initialize?"));
	checkf(Status == EMovieScenePlayerStatus::Stopped, TEXT("This player must be stopped before it can play"));

	// Move the playback position randomly in our playback range if we want a random start time.
	if (bRandomStartTime)
	{
		const int32 RandomStartFrameOffset = FMath::RandHelper(DurationFrames.FrameNumber.Value);
		PlayPosition.Reset(StartFrame + RandomStartFrameOffset);
	}

	// Start playing by evaluating the sequence at the start time.
	bIsLooping = bLoop;
	Status = EMovieScenePlayerStatus::Playing;

	// Unlike the level sequence player, we don't evaluate here because we don't need to: there's
	// no scene to setup or first frame to hold.  We just have to wait for the next update.
}

void UCameraAnimationSequencePlayer::Update(FFrameTime NewPosition)
{
	check(Status == EMovieScenePlayerStatus::Playing);
	check(RootTemplateInstance.IsValid());

	bool bShouldStop = false;

	if (bIsLooping || bDurationRequiresLooping)
	{
		// Unlike the level sequence player, we don't care about making sure to play the last few frames
		// of the sequence before looping: we can jump straight to the looped time because we know we
		// don't have any events to fire or anything like that.
		//
		// Arguably we could have some cumulative animation mode running on some properties but let's call
		// this a limitation for now.
		//
		while (NewPosition >= StartFrame + DurationFrames)
		{
			NewPosition -= DurationFrames;
			PlayPosition.Reset(StartFrame);
			++LoopsPlayed;
		}
	}
	else
	{
		// If we are reaching the end, update the sequence at the end time and stop.
		if (NewPosition >= StartFrame + DurationFrames)
		{
			NewPosition = FFrameTime(StartFrame + DurationFrames);
			bShouldStop = true;
		}
	}

	if (bDurationRequiresLooping)
	{
		// If we are only looping a fixed number of times to meet a certain duration override,
		// we need to check here whether we have reached the end.
		const FFrameTime ElapsedPlayedFrames = (LoopsPlayed * DurationFrames) + NewPosition;
		if (ElapsedPlayedFrames >= TotalDurationFrames)
		{
			bShouldStop = true;
		}
	}

	if (TSharedPtr<FMovieSceneEntitySystemRunner> Runner = RootTemplateInstance.GetRunner())
	{
		const FMovieSceneEvaluationRange Range = PlayPosition.PlayTo(NewPosition);
		const FMovieSceneContext Context(Range, Status);

		Runner->QueueUpdate(Context, RootTemplateInstance.GetRootInstanceHandle());

		// @todo: we should really only call flush _once_ for all camera anim sequences
		Runner->Flush();
	}

	if (bShouldStop)
	{
		Stop();
	}
}

void UCameraAnimationSequencePlayer::Jump(FFrameTime NewPosition)
{
	PlayPosition.JumpTo(NewPosition);
}

void UCameraAnimationSequencePlayer::Stop()
{
	if (Status != EMovieScenePlayerStatus::Stopped)
	{
		Status = EMovieScenePlayerStatus::Stopped;

		PlayPosition.Reset(StartFrame);

		if (TSharedPtr<FMovieSceneEntitySystemRunner> Runner = RootTemplateInstance.GetRunner())
		{
			if (Runner->QueueFinalUpdate(RootTemplateInstance.GetRootInstanceHandle()))
			{
				// @todo: we should really only call flush _once_ for all camera anim sequences
				Runner->Flush();
			}
		}
	}
}
