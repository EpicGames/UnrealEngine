// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraAnimationSequenceSubsystem.h"
#include "Engine/World.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneRootInstantiatorSystem.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneSpawnRegister.h"
#include "Systems/MovieScenePropertyInstantiator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAnimationSequenceSubsystem)

#define LOCTEXT_NAMESPACE "CameraAnimationSequenceSubsystem"

namespace UE
{
namespace MovieScene
{

struct FCameraAnimationInstantiationMutation : IMovieSceneEntityMutation
{
	FInstanceRegistry& InstanceRegistry;
	FBuiltInComponentTypes* BuiltInComponents;
	FCameraAnimationInstantiationMutation(FInstanceRegistry& InInstanceRegistry)
		: InstanceRegistry(InInstanceRegistry)
	{
		BuiltInComponents = FBuiltInComponentTypes::Get();
	}
	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
	{
		// Create all output components and bind objects/components.
		InOutEntityComponentTypes->Set(BuiltInComponents->BoundObject);
	}
	virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const
	{
		const int32 Num = Allocation->Num();
		FEntityAllocationWriteContext WriteContext = FEntityAllocationWriteContext::NewAllocation();
		TComponentReader<FInstanceHandle> InstanceHandles = Allocation->ReadComponents(BuiltInComponents->InstanceHandle);
		TComponentWriter<UObject*> OutBoundObjects = Allocation->WriteComponents(BuiltInComponents->BoundObject, WriteContext);
	
		if (AllocationType.Contains(BuiltInComponents->GenericObjectBinding))
		{
			// Initialize bound objects.
			TComponentReader<FGuid> ObjectBindings = Allocation->ReadComponents(BuiltInComponents->GenericObjectBinding);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				BindObjectImpl(InstanceHandles[Index], ObjectBindings[Index], OutBoundObjects[Index]);
			}

			// @NOTE: Camera animations intentionally do not use bound object resolvers because they explicitly
			//        always use a camera stand in object.
		}
	}

	void BindObjectImpl(const FInstanceHandle& InstanceHandle, const FGuid& ObjectBinding, UObject*& OutBoundObject) const
	{
		const FSequenceInstance& Instance = InstanceRegistry.GetInstance(InstanceHandle);
		TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
		TArrayView<TWeakObjectPtr<>> BoundObjects = SharedPlaybackState->FindBoundObjects(ObjectBinding, Instance.GetSequenceID());
		if (ensure(BoundObjects.Num() > 0))
		{
			// In theory we should get the scene component from the object, but we know that camera animations are
			// always played on a camera animation player's "stand-in" camera object, which isn't really a camera, and
			// certainly isn't an actor. It has its transform information directly on itself.
			OutBoundObject = BoundObjects[0].Get();
		}
	}
};

} // namespace MovieScene
} // namespace UE

UCameraAnimationSpawnableSystem::UCameraAnimationSpawnableSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Spawn;
	SystemCategories = UCameraAnimationSequenceSubsystem::GetCameraAnimationSystemCategory();
}

bool UCameraAnimationSpawnableSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;
	return InLinker->GetLinkerRole() == EEntitySystemLinkerRole::CameraAnimations;
}

void UCameraAnimationSpawnableSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->SpawnableBinding)
	.FilterAll({ BuiltInComponents->Tags.NeedsLink })
	.Iterate_PerEntity(&Linker->EntityManager, [InstanceRegistry](const FInstanceHandle& InstanceHandle, const FGuid& SpawnableBinding)
		{
			// We won't actually be spawning anything, because our player's spawn register will simply 
			// return the fake camera "stand-in" object.
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);
			TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
			const UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(Instance.GetSequenceID());
			FMovieSceneSpawnRegister* SpawnRegister = SharedPlaybackState->FindCapability<FMovieSceneSpawnRegister>();
			UObject* SpawnedObject = SpawnRegister->SpawnObject(
				SpawnableBinding, *Sequence->GetMovieScene(), Instance.GetSequenceID(), SharedPlaybackState);
			ensure(SpawnedObject);
		});
}

UCameraAnimationBoundObjectInstantiator::UCameraAnimationBoundObjectInstantiator(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Instantiation;
	SystemCategories = UCameraAnimationSequenceSubsystem::GetCameraAnimationSystemCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		DefineComponentProducer(GetClass(), BuiltInComponents->BoundObject);
	}
}

bool UCameraAnimationBoundObjectInstantiator::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;
	return InLinker->GetLinkerRole() == EEntitySystemLinkerRole::CameraAnimations;
}

void UCameraAnimationBoundObjectInstantiator::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Initialize all new allocations with bound objects and output components.
	FCameraAnimationInstantiationMutation Mutation(*Linker->GetInstanceRegistry());
	FEntityComponentFilter Filter = FEntityComponentFilter()
		.Any({ BuiltInComponents->GenericObjectBinding })
		.All({ BuiltInComponents->InstanceHandle, BuiltInComponents->Tags.NeedsLink })
		.None({ BuiltInComponents->Tags.NeedsUnlink });
	Linker->EntityManager.MutateAll(Filter, Mutation, EMutuallyInclusiveComponentType::All);
}

UCameraAnimationEntitySystemLinker::UCameraAnimationEntitySystemLinker(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SetLinkerRole(EEntitySystemLinkerRole::CameraAnimations);
	GetSystemFilter().SetAllowedCategories(
			// Eval time system, hierarchical bias systems, etc.
			EEntitySystemCategory::Core |
			// Systems to evaluate, blend, and set properties on bound objects
			EEntitySystemCategory::ChannelEvaluators |
			EEntitySystemCategory::BlenderSystems |
			EEntitySystemCategory::PropertySystems |
			// Our custom systems that avoid all the bound object instantiation that duplicates entities
			UCameraAnimationSequenceSubsystem::GetCameraAnimationSystemCategory());
	// Add the property instantiator specifically
	GetSystemFilter().AllowSystem(UMovieScenePropertyInstantiatorSystem::StaticClass());

	// NOTE: since we are tightly controlling the systems that are running to animate cameras, we may end
	//       up not supporting new track types or custom features. However, 95% of these extra systems are
	//       expected be part of the common categories that people might want to grow (custom channel 
	//       evaluators, custom blending modes, new property types, etc)
	//       For the last 5%, these custom systems can opt-in to run in camera animations by belonging to
	//       the camera animation system category.
}

UCameraAnimationSequenceSubsystem* UCameraAnimationSequenceSubsystem::GetCameraAnimationSequenceSubsystem(const UWorld* InWorld)
{
	if (InWorld)
	{
		return InWorld->GetSubsystem<UCameraAnimationSequenceSubsystem>();
	}

	return nullptr;
}

UMovieSceneEntitySystemLinker* UCameraAnimationSequenceSubsystem::CreateLinker(UObject* Outer, FName Name)
{
	return NewObject<UCameraAnimationEntitySystemLinker>(Outer, Name);
}

UE::MovieScene::EEntitySystemCategory UCameraAnimationSequenceSubsystem::GetCameraAnimationSystemCategory()
{
	using namespace UE::MovieScene;
	static EEntitySystemCategory CameraAnimationCategory = UMovieSceneEntitySystem::RegisterCustomSystemCategory();
	return CameraAnimationCategory;
}

UCameraAnimationSequenceSubsystem::UCameraAnimationSequenceSubsystem()
{
}

UCameraAnimationSequenceSubsystem::~UCameraAnimationSequenceSubsystem()
{
}

void UCameraAnimationSequenceSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
}

void UCameraAnimationSequenceSubsystem::Deinitialize()
{
	Runner = nullptr;
	Linker = nullptr;

	Super::Deinitialize();
}

UMovieSceneEntitySystemLinker* UCameraAnimationSequenceSubsystem::GetLinker(bool bAutoCreate)
{
	if (!Linker && bAutoCreate)
	{
		Linker = CreateLinker(this, TEXT("CameraAnimationSequenceSubsystemLinker"));
		Runner = Linker->GetRunner();
	}
	return Linker;
}

TSharedPtr<FMovieSceneEntitySystemRunner> UCameraAnimationSequenceSubsystem::GetRunner() const
{
	return Runner;
}

#undef LOCTEXT_NAMESPACE


