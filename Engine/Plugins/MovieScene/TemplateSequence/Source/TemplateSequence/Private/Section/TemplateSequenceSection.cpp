// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/TemplateSequenceSection.h"
#include "Channels/MovieSceneChannelData.h"
#include "TemplateSequence.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "TemplateSequenceComponentTypes.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "Evaluation/MovieSceneRootOverridePath.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateSequenceSection)

#define LOCTEXT_NAMESPACE "UTemplateSequenceSection"

UTemplateSequenceSection::UTemplateSequenceSection(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
{
	SetBlendType(EMovieSceneBlendType::Absolute);
}

void UTemplateSequenceSection::PostInitProperties()
{
	// Template sequences always adopt the same hierarchical bias as their parent sequence so that their
	// animation can blend with any complementary animation set directly on their target object.
	Parameters.HierarchicalBias = 0;
	
	Super::PostInitProperties();
}

void UTemplateSequenceSection::AddPropertyScale(const FTemplateSectionPropertyScale& InPropertyScale)
{
	PropertyScales.Add(InPropertyScale);
	ChannelProxy = nullptr;
}

void UTemplateSequenceSection::RemovePropertyScale(int32 InIndex)
{
	if (ensure(PropertyScales.IsValidIndex(InIndex)))
	{
		PropertyScales.RemoveAt(InIndex);
		ChannelProxy = nullptr;
	}
}

bool UTemplateSequenceSection::ShowCurveForChannel(const void *ChannelPtr) const
{
	return true;
}

void UTemplateSequenceSection::OnDilated(float DilationFactor, FFrameNumber Origin)
{
	// TODO-lchabant: shouldn't this be in the base class?
	Parameters.TimeScale.ScaleBy(1.f / DilationFactor);
}

EMovieSceneChannelProxyType UTemplateSequenceSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

	for (FTemplateSectionPropertyScale& PropertyScale : PropertyScales)
	{
#if WITH_EDITOR
		FName ChannelName = PropertyScale.PropertyBinding.PropertyPath;
		switch (PropertyScale.PropertyScaleType)
		{
			case ETemplateSectionPropertyScaleType::TransformPropertyLocationOnly:
				ChannelName = FName(*FString::Format(TEXT("{0}[Location]"), { ChannelName.ToString() }));
				break;
			case ETemplateSectionPropertyScaleType::TransformPropertyRotationOnly:
				ChannelName = FName(*FString::Format(TEXT("{0}[Rotation]"), { ChannelName.ToString() }));
				break;
			default:
				break;
		}

		FMovieSceneChannelMetaData ChannelMetaData;
		ChannelMetaData.Name = ChannelName;
		ChannelMetaData.bCanCollapseToTrack = false;
		ChannelMetaData.DisplayText = FText::Format(LOCTEXT("PropertyScaleDisplayText", "{0} Multiplier"), FText::FromName(ChannelName));
		Channels.Add(PropertyScale.FloatChannel, ChannelMetaData, TMovieSceneExternalValue<float>());

#else

		Channels.Add(PropertyScale.FloatChannel);

#endif
	}

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));

	return EMovieSceneChannelProxyType::Dynamic;	
}

bool UTemplateSequenceSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const int32 EntityIndex   = OutFieldBuilder->FindOrAddEntity(this, 0);
	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);
	OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);

	// Add extra entities for each property scale.
	for (int32 Index = 0; Index < PropertyScales.Num(); ++Index)
	{
		const FTemplateSectionPropertyScale& PropertyScale(PropertyScales[Index]);
		const int32 PropertyScaleEntityIndex = OutFieldBuilder->FindOrAddEntity(this, Index + 1);
		const int32 PropertyScaleMetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, PropertyScaleEntityIndex, PropertyScaleMetaDataIndex);
	}

	return true;
}

void UTemplateSequenceSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	UTemplateSequence* TemplateSubSequence = Cast<UTemplateSequence>(GetSequence());
	if (!TemplateSubSequence)
	{
		return;
	}

	if (!EntityLinker->GetInstanceRegistry()->IsHandleValid(Params.Sequence.InstanceHandle))
	{
		return;
	}

	const FSubSequencePath PathToRoot = EntityLinker->GetInstanceRegistry()->GetInstance(Params.Sequence.InstanceHandle).GetSubSequencePath();
	const FMovieSceneSequenceID ResolvedSequenceID = PathToRoot.ResolveChildSequenceID(GetSequenceID());

	if (Params.EntityID == 0)
	{
		// Add the template component to our entity. This component basically just stores the inner (template)
		// sub-sequence's root object binding as a resolved-from-root ID, so we can setup an object resolution override
		// on it when the main (root) sequence starts.
		FTemplateSequenceComponentData ComponentData;
		ComponentData.InnerOperand = FMovieSceneEvaluationOperand(ResolvedSequenceID, TemplateSubSequence->GetRootObjectBindingID());

		const FGuid ObjectBindingID = Params.GetObjectBindingID();

		OutImportedEntity->AddBuilder(
			FEntityBuilder()
				.AddConditional(FBuiltInComponentTypes::Get()->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
				.Add(FTemplateSequenceComponentTypes::Get()->TemplateSequence, ComponentData)
				);

		BuildDefaultSubSectionComponents(EntityLinker, Params, OutImportedEntity);
	}
	else if (ensure(PropertyScales.IsValidIndex(Params.EntityID - 1)))
	{
		// Add property scale entity for one of our property scales.
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		const FTemplateSequenceComponentTypes* TemplateSequenceComponents = FTemplateSequenceComponentTypes::Get();

		const FTemplateSectionPropertyScale& PropertyScale(PropertyScales[Params.EntityID - 1]);
		const bool bHasScale = (
				PropertyScale.FloatChannel.GetNumKeys() > 0 || PropertyScale.FloatChannel.GetDefault().Get(1.f) != 1.f);
		if (bHasScale)
		{
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
					.Add(
						TemplateSequenceComponents->PropertyScale,
						FTemplateSequencePropertyScaleComponentData
						{
							ResolvedSequenceID,
							PropertyScale.ObjectBinding, 
							PropertyScale.PropertyBinding,
							PropertyScale.PropertyScaleType
						})
					.Add(
						BuiltInComponents->FloatChannel[0],
						FSourceFloatChannel(&PropertyScale.FloatChannel))
					.AddMutualComponents());
		}
	}
}

#undef LOCTEXT_NAMESPACE

