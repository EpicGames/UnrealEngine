// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceCameraShake.h"
#include "CameraAnimationSequence.h"
#include "CameraAnimationSequencePlayer.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "MovieScene.h"
#include "MovieSceneFwd.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequenceCameraShake)


USequenceCameraShakePattern::USequenceCameraShakePattern(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, PlayRate(1.f)
	, Scale(1.f)
	, BlendInTime(0.2f)
	, BlendOutTime(0.4f)
	, RandomSegmentDuration(0.f)
	, bRandomSegment(false)
{
	CameraStandIn = CreateDefaultSubobject<UCameraAnimationSequenceCameraStandIn>(TEXT("CameraStandIn"), true);
	Player = CreateDefaultSubobject<UCameraAnimationSequencePlayer>(TEXT("CameraShakePlayer"), true);
}

void USequenceCameraShakePattern::GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const
{
	if (Sequence != nullptr)
	{
		if (UMovieScene* MovieScene = Sequence->GetMovieScene())
		{
			const float ActualPlayRate = (PlayRate > 0.f) ? PlayRate : 1.f;

			if (bRandomSegment)
			{
				OutInfo.Duration = FCameraShakeDuration(RandomSegmentDuration / ActualPlayRate);
			}
			else
			{
				const FFrameRate TickResolution = MovieScene->GetTickResolution();
				const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
				const float Duration = TickResolution.AsSeconds(PlaybackRange.Size<FFrameNumber>());
				OutInfo.Duration = FCameraShakeDuration(Duration / ActualPlayRate);
			}

			OutInfo.BlendIn = BlendInTime;
			OutInfo.BlendOut = BlendOutTime;
		}
	}
}

void USequenceCameraShakePattern::StartShakePatternImpl(const FCameraShakePatternStartParams& Params)
{
	using namespace UE::MovieScene;

	if (!ensure(Sequence))
	{
		return;
	}

	// Initialize our stand-in object.
	CameraStandIn->Initialize(Sequence);
	
	// Make the player always use our stand-in object whenever a sequence wants to spawn or possess an object.
	Player->SetBoundObjectOverride(CameraStandIn);

	// Initialize it and start playing.
	float DurationOverride = 0.f;
	if (Params.bOverrideDuration)
	{
		DurationOverride = Params.DurationOverride;
	}
	Player->Initialize(Sequence, 0, DurationOverride);
	Player->Play(bRandomSegment, bRandomSegment);

	// Initialize our state.
	State.Start(this, Params);
}

void USequenceCameraShakePattern::UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	const float BlendWeight = State.Update(Params.DeltaTime);
	if (State.IsPlaying())
	{
		const FFrameRate DisplayRate = Player->GetInputRate();
		const FFrameTime CurrentPosition = Player->GetCurrentPosition();

		const FFrameTime NewPosition = CurrentPosition + Params.DeltaTime * PlayRate * DisplayRate;
		UpdateCamera(NewPosition, Params.POV, OutResult);

		OutResult.ApplyScale(BlendWeight);
	}
	else
	{
		// We could have stopped playing if we were blending out as a result of a call to Stop(false)
		// In this case, the sequence might still be playing (i.e. in a valid position inside its playback
		// range) and we need to explicitly stop it.
		check(Player);
		Player->Stop();
	}
}

void USequenceCameraShakePattern::ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult)
{
	const float BlendWeight = State.Scrub(Params.AbsoluteTime);
	if (State.IsPlaying())
	{
		if (Player->GetPlaybackStatus() != EMovieScenePlayerStatus::Playing)
		{
			Player->Play(bRandomSegment, bRandomSegment);
		}

		const FFrameRate InputRate = Player->GetInputRate();
		const FFrameTime NewPosition = Params.AbsoluteTime * PlayRate * InputRate;
		UpdateCamera(NewPosition, Params.POV, OutResult);

		OutResult.ApplyScale(BlendWeight);
	}
	else
	{
		// See the similar else clause in UpdateShakePatternImpl. 
		check(Player);
		Player->Stop();
	}
}

bool USequenceCameraShakePattern::IsFinishedImpl() const
{
	return (Player == nullptr || Player->GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped);
}

void USequenceCameraShakePattern::StopShakePatternImpl(const FCameraShakePatternStopParams& Params)
{
	using namespace UE::MovieScene;

	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (ensure(Player != nullptr && MovieScene != nullptr))
	{
		if (Params.bImmediately)
		{
			// Stop playing!
			Player->Stop();
		}

		// Make our state tracking stop or start blending out.
		State.Stop(Params.bImmediately);
	}
}

void USequenceCameraShakePattern::TeardownShakePatternImpl()
{
	using namespace UE::MovieScene;

	// Stop if we had reached the end of the animation and the sequence needs finishing.
	// If the shake had been stopped explicitly, this basically won't do anything.
	Player->Stop();

	// Reset our time tracking.
	State = FCameraShakeState();
}

void USequenceCameraShakePattern::UpdateCamera(FFrameTime NewPosition, const FMinimalViewInfo& InPOV, FCameraShakePatternUpdateResult& OutResult)
{
	if (!ensure(Sequence))
	{
		return;
	}

	if (!ensure(Player) || !Player->IsValid())
	{
		return;
	}

	using namespace UE::MovieScene;

	check(CameraStandIn);

	UMovieSceneEntitySystemLinker* Linker = Player->GetEvaluationTemplate().GetEntitySystemLinker();
	CameraStandIn->Reset(InPOV, Linker);

	// Get the "unshaken" properties that need to be treated additively.
	const float OriginalFieldOfView = CameraStandIn->FieldOfView;

	// Update the sequence.
	Player->Update(NewPosition);

	// Recalculate properties that might be invalidated by other properties having been animated.
	CameraStandIn->RecalcDerivedData();

	// Grab the final animated (shaken) values, figure out the delta, apply scale, and feed that into the 
	// camera shake result.
	// Transform is always treated as a local, additive value. The data better be good.
	const FTransform ShakenTransform = CameraStandIn->GetTransform();
	OutResult.Location = ShakenTransform.GetLocation() * Scale;
	OutResult.Rotation = ShakenTransform.GetRotation().Rotator() * Scale;

	// FieldOfView follows the current camera's value every frame, so we can compute how much the shake is
	// changing it.
	const float ShakenFieldOfView = CameraStandIn->FieldOfView;
	const float DeltaFieldOfView = ShakenFieldOfView - OriginalFieldOfView;
	OutResult.FOV = DeltaFieldOfView * Scale;

	// The other properties aren't treated as additive.
	OutResult.PostProcessSettings = CameraStandIn->PostProcessSettings;
	OutResult.PostProcessBlendWeight = CameraStandIn->PostProcessBlendWeight;
}


