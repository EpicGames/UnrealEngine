// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/TemplateSequenceSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/TemplateSequenceSection.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieScene3DTransformPropertySystem.h"
#include "Systems/MovieSceneBaseValueEvaluatorSystem.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/MovieSceneFloatPropertySystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "TemplateSequenceComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateSequenceSystem)

UTemplateSequenceSystem::UTemplateSequenceSystem(const FObjectInitializer& ObjInit)
	: UMovieSceneEntitySystem(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FTemplateSequenceComponentTypes* TemplateSequenceComponents = FTemplateSequenceComponentTypes::Get();

	Phase = UE::MovieScene::ESystemPhase::Spawn;
	RelevantComponent = TemplateSequenceComponents->TemplateSequence;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(GetClass(), UMovieSceneSpawnablesSystem::StaticClass());
	}

	// We only need to run if there are template sequence sections starting or stopping.
	ApplicableFilter.Filter.All({ TemplateSequenceComponents->TemplateSequence });
	ApplicableFilter.Filter.Any({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink });
}

void UTemplateSequenceSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	// Only run if we must.
	if (!ApplicableFilter.Matches(Linker->EntityManager))
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FTemplateSequenceComponentTypes* TemplateSequenceComponents = FTemplateSequenceComponentTypes::Get();

	FInstanceRegistry* InstanceRegistry  = Linker->GetInstanceRegistry();

	auto SetupTeardownBindingOverrides = [BuiltInComponents, InstanceRegistry](
			FEntityAllocationIteratorItem AllocationItem,
			TRead<FInstanceHandle> InstanceHandles,
			TRead<FGuid> ObjectBindingIDs,
			TRead<FTemplateSequenceComponentData> TemplateSequenceDatas)
	{
		const FComponentMask& Mask = AllocationItem.GetAllocationType();
		const bool bHasNeedsLink   = Mask.Contains(BuiltInComponents->Tags.NeedsLink);
		const bool bHasNeedsUnlink = Mask.Contains(BuiltInComponents->Tags.NeedsUnlink);

		const int32 Num = AllocationItem.GetAllocation()->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandles[Index]);
			const FGuid& ObjectBindingID = ObjectBindingIDs[Index];
			const FTemplateSequenceComponentData& TemplateSequenceData = TemplateSequenceDatas[Index];

			IStaticBindingOverridesPlaybackCapability* StaticOverrides = SequenceInstance.GetSharedPlaybackState()->FindCapability<IStaticBindingOverridesPlaybackCapability>();
			if (ensure(StaticOverrides))
			{
				if (bHasNeedsLink)
				{
					const FMovieSceneSequenceID SequenceID = SequenceInstance.GetSequenceID();
					const FMovieSceneEvaluationOperand OuterOperand(SequenceID, ObjectBindingID);
					StaticOverrides->AddBindingOverride(TemplateSequenceData.InnerOperand, OuterOperand);
				}
				else if (bHasNeedsUnlink)
				{
					StaticOverrides->RemoveBindingOverride(TemplateSequenceData.InnerOperand);
				}
			}
		}
	};

	FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->GenericObjectBinding)
		.Read(TemplateSequenceComponents->TemplateSequence)
		.FilterAny({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerAllocation(&Linker->EntityManager, SetupTeardownBindingOverrides);
}

UTemplateSequencePropertyScalingInstantiatorSystem::UTemplateSequencePropertyScalingInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FTemplateSequenceComponentTypes* TemplateSequenceComponents = FTemplateSequenceComponentTypes::Get();

	// We run during the instantiation phase if there are any template sequence properties to scale.
	Phase = ESystemPhase::Instantiation;
	RelevantComponent = TemplateSequenceComponents->PropertyScale;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// We need to run after binding IDs have been resolved to bound objects.
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
	}
}

void UTemplateSequencePropertyScalingInstantiatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FTemplateSequenceComponentTypes* TemplateSequenceComponents = FTemplateSequenceComponentTypes::Get();

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	// Step 1: Keep track of any new property scales.
	auto GatherNewPropertyScaledInstances = [this, InstanceRegistry](
			const FMovieSceneEntityID EntityID,
			const FRootInstanceHandle RootInstanceHandle,
			const FTemplateSequencePropertyScaleComponentData& PropertyScale)
	{
		FPropertyScaledInstanceKey Key(RootInstanceHandle, PropertyScale.SubSequenceID);

		FPropertyScaleEntityIDs& PropertyScaleEntities = PropertyScaledInstances.FindOrAdd(Key);
		if (!PropertyScaleEntities.Contains(EntityID))
		{
			PropertyScaleEntities.Add(EntityID);

			switch (PropertyScale.PropertyScaleType)
			{
				case ETemplateSectionPropertyScaleType::FloatProperty:
					++FloatScaleUseCount;
					break;
				case ETemplateSectionPropertyScaleType::TransformPropertyLocationOnly:
				case ETemplateSectionPropertyScaleType::TransformPropertyRotationOnly:
					++TransformScaleUseCount;
					break;
			}
		}
	};

	FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(TemplateSequenceComponents->PropertyScale)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.Iterate_PerEntity(&Linker->EntityManager, GatherNewPropertyScaledInstances);

	// Step 2: Remove old property scales ending.
	auto RemoveOldPropertyScaledInstances = [this, InstanceRegistry](
			const FMovieSceneEntityID EntityID,
			const FRootInstanceHandle RootInstanceHandle,
			const FTemplateSequencePropertyScaleComponentData& PropertyScale)
	{
		FPropertyScaledInstanceKey Key(RootInstanceHandle, PropertyScale.SubSequenceID);

		if (FPropertyScaleEntityIDs* PropertyScaleEntities = PropertyScaledInstances.Find(Key))
		{
			if (PropertyScaleEntities->Remove(EntityID) > 0)
			{
				switch (PropertyScale.PropertyScaleType)
				{
					case ETemplateSectionPropertyScaleType::FloatProperty:
						--FloatScaleUseCount;
						break;
					case ETemplateSectionPropertyScaleType::TransformPropertyLocationOnly:
					case ETemplateSectionPropertyScaleType::TransformPropertyRotationOnly:
						--TransformScaleUseCount;
						break;
				}
			}
			if (PropertyScaleEntities->Num() == 0)
			{
				PropertyScaledInstances.Remove(Key);
			}
		}
	};

	FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(TemplateSequenceComponents->PropertyScale)
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink, TemplateSequenceComponents->PropertyScale })
		.Iterate_PerEntity(&Linker->EntityManager, RemoveOldPropertyScaledInstances);

	// Step 3: Look at all new entities starting this frame, and set aside all those who belong to the active 
	// template sub-sequences that we have identified in steps 1 & 2. While we do that, we also lookup these 
	// entities' original binding ID (if they have one) by reading it from their parent entity.
	TArray<TTuple<FMovieSceneEntityID, FGuid>> PropertyScaledEntities;

	auto ComputeReverseLookupBinding = [this, BuiltInComponents, &PropertyScaledEntities](
			const FMovieSceneEntityID EntityID,
			const FRootInstanceHandle RootInstanceHandle,
			const FMovieSceneSequenceID* OptSequenceID,
			const FMovieSceneEntityID ParentEntity, 
			UObject* const BoundObject)
	{
		FMovieSceneSequenceID SequenceID = OptSequenceID ? *OptSequenceID : MovieSceneSequenceID::Root;
		if (!PropertyScaledInstances.Contains(FPropertyScaledInstanceKey(RootInstanceHandle, SequenceID)))
		{
			return;
		}

		if (ParentEntity.IsValid())
		{
			TOptionalComponentReader<FGuid> Reader = Linker->EntityManager.ReadComponent(ParentEntity, BuiltInComponents->GenericObjectBinding);
			if (ensure(Reader.IsValid()))
			{
				const FGuid ObjectBindingID = *Reader.ComponentAtIndex(0);
				PropertyScaledEntities.Emplace(EntityID, ObjectBindingID);
			}
		}
	};

	FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.ReadOptional(BuiltInComponents->SequenceID)
		.Read(BuiltInComponents->ParentEntity)
		.Read(BuiltInComponents->BoundObject)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.Iterate_PerEntity(&Linker->EntityManager, ComputeReverseLookupBinding);

	// Step 4: Store the original binding ID of the entities identified in the previous step, i.e. the entities that
	// belong to an active template sub-sequence that has at least some properties scaled.
	for (const TTuple<FMovieSceneEntityID, FGuid>& Pair : PropertyScaledEntities)
	{
		FEntityBuilder()
		.Add(TemplateSequenceComponents->PropertyScaleReverseBindingLookup, Pair.Get<1>())
		// Stop constants from being optimized since we need them to be re-computed every frame to remove the scale
		.AddTag(BuiltInComponents->Tags.DontOptimizeConstants)
		.MutateExisting(&Linker->EntityManager, Pair.Get<0>());
	}
}

UTemplateSequencePropertyScalingEvaluatorSystem::UTemplateSequencePropertyScalingEvaluatorSystem(const FObjectInitializer& ObjInit)
	: UMovieSceneEntitySystem(ObjInit)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FTemplateSequenceComponentTypes* TemplateSequenceComponents = FTemplateSequenceComponentTypes::Get();

	Phase = ESystemPhase::Scheduling;
	RelevantComponent = TemplateSequenceComponents->PropertyScale;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// We need to wait until float and double channels have evaluated (both the values that we're going to multiply, and the
		// channel of the scale multiplier itself).
		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBaseValueEvaluatorSystem::StaticClass(), GetClass());
		// We need to multiply values before they are blended together with other values we shouldn't touch.
		DefineImplicitPrerequisite(GetClass(), UMovieScenePiecewiseDoubleBlenderSystem::StaticClass());
		// We need to multiply values before they are set on the animated objects. For blended values, the dependencies above are
		// enough, but for non-blended values that use the "fast-path", we need this dependency.
		DefineImplicitPrerequisite(GetClass(), UMovieSceneFloatPropertySystem::StaticClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneComponentTransformSystem::StaticClass());
		DefineImplicitPrerequisite(GetClass(), UMovieScene3DTransformPropertySystem::StaticClass());
		// We need this component to lookup the binding GUID for an entity, so we know what scaling to apply to it.
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
	}
}

void UTemplateSequencePropertyScalingEvaluatorSystem::ResetPropertyScales()
{
	PropertyScales.Reset();
}

void UTemplateSequencePropertyScalingEvaluatorSystem::AddPropertyScale(const FPropertyScaleKey& Key, const FPropertyScaleValue& Value)
{
	FMultiPropertyScaleValue& MultiValue = PropertyScales.FindOrAdd(Key);

	// In most cases there's only 1 multiplier for a given property, but there can be 2 multipliers in the case of 
	// a location + rotation scale on a transform property.
	if (ensure(MultiValue.Values.Num() < 2))
	{
		MultiValue.Values.Add(Value);
	}
}

void UTemplateSequencePropertyScalingEvaluatorSystem::FindPropertyScales(const FPropertyScaleKey& Key, FPropertyScaleValueArray& OutValues) const
{
	if (const FMultiPropertyScaleValue* MultiValue = PropertyScales.Find(Key))
	{
		ensure(MultiValue->Values.Num() <= 2);
		OutValues.Append(MultiValue->Values);
	}
}

namespace UE
{
namespace MovieScene
{

template<typename ValueType, typename WriteAccessor>
void ScalePropertyValue(float InScale, TReadOptional<ValueType> InBaseValues, WriteAccessor InOutValues, int32 Index)
{
	if (InOutValues)
	{
		if (InBaseValues)
		{
			const float BaseValue = InBaseValues[Index];
			InOutValues[Index] = (InOutValues[Index] - BaseValue) * InScale + BaseValue;
		}
		else
		{
			InOutValues[Index] *= InScale;
		}
	}
}

struct FGatherPropertyScales
{
	using UEvaluatorSystem = UTemplateSequencePropertyScalingEvaluatorSystem;

	UEvaluatorSystem* EvaluatorSystem;
	FInstanceRegistry* InstanceRegistry;

	FGatherPropertyScales(UEvaluatorSystem* InEvaluatorSystem)
		: EvaluatorSystem(InEvaluatorSystem)
	{
		check(InEvaluatorSystem);
		InstanceRegistry = InEvaluatorSystem->GetLinker()->GetInstanceRegistry();
	}

	void PreTask()
	{
		// Clear our map of property scales from last frame.
		EvaluatorSystem->ResetPropertyScales();
	}

	void ForEachEntity(
			const FRootInstanceHandle& RootInstanceHandle, 
			const FTemplateSequencePropertyScaleComponentData& PropertyScale,
			float ScaleFactor) const
	{
		// Find the instance handle of the sub-sequence we need to scale.
		EvaluatorSystem->AddPropertyScale(
				UEvaluatorSystem::FPropertyScaleKey { RootInstanceHandle, PropertyScale.SubSequenceID, PropertyScale.ObjectBinding, PropertyScale.PropertyBinding.PropertyPath },
				UEvaluatorSystem::FPropertyScaleValue { PropertyScale.PropertyScaleType, ScaleFactor });
	}
};

struct FScaleTransformProperties
{
	using UEvaluatorSystem = UTemplateSequencePropertyScalingEvaluatorSystem;

	const UEvaluatorSystem* EvaluatorSystem;

	FScaleTransformProperties(const UEvaluatorSystem* InEvaluatorSystem)
		: EvaluatorSystem(InEvaluatorSystem)
	{
		check(InEvaluatorSystem);
	}

	void ForEachAllocation(
			FEntityAllocation* Allocation,
			TRead<FRootInstanceHandle> RootInstanceHandles,
			TReadOptional<FMovieSceneSequenceID> SequenceIDs,
			TRead<FMovieScenePropertyBinding> PropertyBindings,
			TRead<FGuid> ReverseBindingLookups,
			TReadOptional<double> BaseLocationXs, TReadOptional<double> BaseLocationYs, TReadOptional<double> BaseLocationZs,
			TReadOptional<double> BaseRotationXs, TReadOptional<double> BaseRotationYs, TReadOptional<double> BaseRotationZs,
			TWriteOptional<double> LocationXs,
			TWriteOptional<double> LocationYs,
			TWriteOptional<double> LocationZs,
			TWriteOptional<double> RotationXs,
			TWriteOptional<double> RotationYs,
			TWriteOptional<double> RotationZs) const
	{
		for (int32 Index = 0; Index < Allocation->Num(); ++Index)
		{
			const FRootInstanceHandle InstanceHandle = RootInstanceHandles[Index];
			const FMovieScenePropertyBinding& PropertyBinding = PropertyBindings[Index];
			const FGuid ObjectBindingID = ReverseBindingLookups[Index];
			const FMovieSceneSequenceID SequenceID = SequenceIDs ? SequenceIDs[Index] : MovieSceneSequenceID::Root;

			ensure(ObjectBindingID.IsValid());

			UEvaluatorSystem::FPropertyScaleValueArray TransformScales;
			EvaluatorSystem->FindPropertyScales(
					UEvaluatorSystem::FPropertyScaleKey { InstanceHandle, SequenceID, ObjectBindingID, PropertyBinding.PropertyPath },
					TransformScales);
			for (const UEvaluatorSystem::FPropertyScaleValue& TransformScale : TransformScales)
			{
				const float ScaleFactor = TransformScale.Get<1>();

				switch (TransformScale.Get<0>())
				{
					case ETemplateSectionPropertyScaleType::TransformPropertyLocationOnly:
						ScalePropertyValue(ScaleFactor, BaseLocationXs, LocationXs, Index);
						ScalePropertyValue(ScaleFactor, BaseLocationYs, LocationYs, Index);
						ScalePropertyValue(ScaleFactor, BaseLocationZs, LocationZs, Index);
						break;

					case ETemplateSectionPropertyScaleType::TransformPropertyRotationOnly:
						ScalePropertyValue(ScaleFactor, BaseRotationXs, RotationXs, Index);
						ScalePropertyValue(ScaleFactor, BaseRotationYs, RotationYs, Index);
						ScalePropertyValue(ScaleFactor, BaseRotationZs, RotationZs, Index);
						break;

					default:
						ensureMsgf(false, TEXT("Unsupported or invalid transform property scale type."));
						break;
				}
			}
		}
	}
};

struct FScaleFloatProperties
{
	using UEvaluatorSystem = UTemplateSequencePropertyScalingEvaluatorSystem;

	const UEvaluatorSystem* EvaluatorSystem;

	FScaleFloatProperties(const UEvaluatorSystem* InEvaluatorSystem)
		: EvaluatorSystem(InEvaluatorSystem)
	{
		check(InEvaluatorSystem);
	}

	void ForEachAllocation(
			FEntityAllocation* Allocation,
			TRead<FRootInstanceHandle> RootInstanceHandles,
			TReadOptional<FMovieSceneSequenceID> SequenceIDs,
			TRead<FMovieScenePropertyBinding> PropertyBindings,
			TRead<FGuid> ReverseBindingLookups,
			TReadOptional<double> BasePropertyValues,
			TWrite<double> PropertyValues) const
	{
		for (int32 Index = 0; Index < Allocation->Num(); ++Index)
		{
			const FRootInstanceHandle InstanceHandle = RootInstanceHandles[Index];
			const FMovieScenePropertyBinding& PropertyBinding = PropertyBindings[Index];
			const FGuid ObjectBindingID = ReverseBindingLookups[Index];
			const FMovieSceneSequenceID SequenceID = SequenceIDs ? SequenceIDs[Index] : MovieSceneSequenceID::Root;
			ensure(ObjectBindingID.IsValid());

			UEvaluatorSystem::FPropertyScaleValueArray PropertyScales;
			EvaluatorSystem->FindPropertyScales(
					UEvaluatorSystem::FPropertyScaleKey { InstanceHandle, SequenceID, ObjectBindingID, PropertyBinding.PropertyPath },
					PropertyScales);
			for (const UEvaluatorSystem::FPropertyScaleValue& PropertyScale : PropertyScales)
			{
				const float ScaleFactor = PropertyScale.Get<1>();
				switch (PropertyScale.Get<0>())
				{
					case ETemplateSectionPropertyScaleType::FloatProperty:
						ScalePropertyValue(ScaleFactor, BasePropertyValues, PropertyValues, Index);
						break;
					default:
						ensureMsgf(false, TEXT("Unsupported or invalid float property scale type."));
						break;
				}
			}
		}
	}
};

} // namespace MovieScene
} // namespace UE

void UTemplateSequencePropertyScalingEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	const FTemplateSequenceComponentTypes* TemplateSequenceComponents = FTemplateSequenceComponentTypes::Get();

	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	const UTemplateSequencePropertyScalingInstantiatorSystem* InstantiatorSystem = Linker->FindSystem<UTemplateSequencePropertyScalingInstantiatorSystem>();

	// Step 1: We are going to look for all entities that describe an active property scale, pick up its up-to-date (evaluated)
	// scale value (which was evaluated by the float channel evaluator), and build a map that tells us:
	// - What properties (sub-sequence instance, object/component binding, property path)...
	// - ...are scaled by what factor (scale value), using what type of scale (scale type enum).
	//
	FTaskID GatherTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(TemplateSequenceComponents->PropertyScale)
	.Read(BuiltInComponents->DoubleResult[0])
	.Schedule_PerEntity<FGatherPropertyScales>(&Linker->EntityManager, TaskScheduler, this);


	// Step 2: If we have any transform property scaling, iterate specifically on them and apply the scale factor
	// to the appropriate components.
	if (ensure(InstantiatorSystem) && InstantiatorSystem->HasAnyTransformScales())
	{
		FTaskID ScaleTransformTask = FEntityTaskBuilder()
			.Read(BuiltInComponents->RootInstanceHandle)
			.ReadOptional(BuiltInComponents->SequenceID)
			.Read(BuiltInComponents->PropertyBinding)
			.Read(TemplateSequenceComponents->PropertyScaleReverseBindingLookup)
			.ReadOptional(BuiltInComponents->BaseDouble[0])
			.ReadOptional(BuiltInComponents->BaseDouble[1])
			.ReadOptional(BuiltInComponents->BaseDouble[2])
			.ReadOptional(BuiltInComponents->BaseDouble[3])
			.ReadOptional(BuiltInComponents->BaseDouble[4])
			.ReadOptional(BuiltInComponents->BaseDouble[5])
			.WriteOptional(BuiltInComponents->DoubleResult[0])
			.WriteOptional(BuiltInComponents->DoubleResult[1])
			.WriteOptional(BuiltInComponents->DoubleResult[2])
			.WriteOptional(BuiltInComponents->DoubleResult[3])
			.WriteOptional(BuiltInComponents->DoubleResult[4])
			.WriteOptional(BuiltInComponents->DoubleResult[5])
			.FilterAny({ TrackComponents->ComponentTransform.PropertyTag, TrackComponents->Transform.PropertyTag })
			.Fork_PerAllocation<FScaleTransformProperties>(&Linker->EntityManager, TaskScheduler, this);

		TaskScheduler->AddPrerequisite(GatherTask, ScaleTransformTask);
	}

	// Step 3: Do the same as step 2 for float properties.
	if (ensure(InstantiatorSystem) && InstantiatorSystem->HasAnyFloatScales())
	{
		FTaskID ScaleFloatTask = FEntityTaskBuilder()
			.Read(BuiltInComponents->RootInstanceHandle)
			.ReadOptional(BuiltInComponents->SequenceID)
			.Read(BuiltInComponents->PropertyBinding)
			.Read(TemplateSequenceComponents->PropertyScaleReverseBindingLookup)
			.ReadOptional(BuiltInComponents->BaseDouble[0])
			.Write(BuiltInComponents->DoubleResult[0])
			.FilterAll({ TrackComponents->Float.PropertyTag })
			.Fork_PerAllocation<FScaleFloatProperties>(&Linker->EntityManager, TaskScheduler, this);

		TaskScheduler->AddPrerequisite(GatherTask, ScaleFloatTask);
	}
}

void UTemplateSequencePropertyScalingEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	const FTemplateSequenceComponentTypes* TemplateSequenceComponents = FTemplateSequenceComponentTypes::Get();

	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	const UTemplateSequencePropertyScalingInstantiatorSystem* InstantiatorSystem = Linker->FindSystem<UTemplateSequencePropertyScalingInstantiatorSystem>();

	// Step 0: Clear our map of property scales from last frame.
	PropertyScales.Reset();

	// Step 1: We are going to look for all entities that describe an active property scale, pick up its up-to-date (evaluated)
	// scale value (which was evaluated by the float channel evaluator), and build a map that tells us:
	// - What properties (sub-sequence instance, object/component binding, property path)...
	// - ...are scaled by what factor (scale value), using what type of scale (scale type enum).
	//
	FGraphEventRef GatherTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(TemplateSequenceComponents->PropertyScale)
		.Read(BuiltInComponents->DoubleResult[0])
		.Dispatch_PerEntity<FGatherPropertyScales>(&Linker->EntityManager, InPrerequisites, &Subsequents, this);
	if (GatherTask)
	{
		InPrerequisites.AddRootTask(GatherTask);
	}

	// Step 2: If we have any transform property scaling, iterate specifically on them and apply the scale factor
	// to the appropriate components.
	if (ensure(InstantiatorSystem) && InstantiatorSystem->HasAnyTransformScales())
	{
		FEntityTaskBuilder()
			.Read(BuiltInComponents->RootInstanceHandle)
			.ReadOptional(BuiltInComponents->SequenceID)
			.Read(BuiltInComponents->PropertyBinding)
			.Read(TemplateSequenceComponents->PropertyScaleReverseBindingLookup)
			.ReadOptional(BuiltInComponents->BaseDouble[0])
			.ReadOptional(BuiltInComponents->BaseDouble[1])
			.ReadOptional(BuiltInComponents->BaseDouble[2])
			.ReadOptional(BuiltInComponents->BaseDouble[3])
			.ReadOptional(BuiltInComponents->BaseDouble[4])
			.ReadOptional(BuiltInComponents->BaseDouble[5])
			.WriteOptional(BuiltInComponents->DoubleResult[0])
			.WriteOptional(BuiltInComponents->DoubleResult[1])
			.WriteOptional(BuiltInComponents->DoubleResult[2])
			.WriteOptional(BuiltInComponents->DoubleResult[3])
			.WriteOptional(BuiltInComponents->DoubleResult[4])
			.WriteOptional(BuiltInComponents->DoubleResult[5])
			.FilterAny({ TrackComponents->ComponentTransform.PropertyTag, TrackComponents->Transform.PropertyTag })
			.Dispatch_PerAllocation<FScaleTransformProperties>(&Linker->EntityManager, InPrerequisites, &Subsequents, this);
	}

	// Step 3: Do the same as step 2 for float properties.
	if (ensure(InstantiatorSystem) && InstantiatorSystem->HasAnyFloatScales())
	{
		FEntityTaskBuilder()
			.Read(BuiltInComponents->RootInstanceHandle)
			.ReadOptional(BuiltInComponents->SequenceID)
			.Read(BuiltInComponents->PropertyBinding)
			.Read(TemplateSequenceComponents->PropertyScaleReverseBindingLookup)
			.ReadOptional(BuiltInComponents->BaseDouble[0])
			.Write(BuiltInComponents->DoubleResult[0])
			.FilterAll({ TrackComponents->Float.PropertyTag })
			.Dispatch_PerAllocation<FScaleFloatProperties>(&Linker->EntityManager, InPrerequisites, &Subsequents, this);
	}
}


