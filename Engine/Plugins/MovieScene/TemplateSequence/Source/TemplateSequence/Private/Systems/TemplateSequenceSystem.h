// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "TemplateSequenceSystem.generated.h"

enum class ETemplateSectionPropertyScaleType;

UCLASS(MinimalAPI)
class UTemplateSequenceSystem : public UMovieSceneEntitySystem
{
public:
	GENERATED_BODY()

	UTemplateSequenceSystem(const FObjectInitializer& ObjInit);

private:
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

private:
	UE::MovieScene::FCachedEntityFilterResult_Match ApplicableFilter;
};

UCLASS(MinimalAPI)
class UTemplateSequencePropertyScalingInstantiatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UTemplateSequencePropertyScalingInstantiatorSystem(const FObjectInitializer& ObjInit);

	bool HasAnyFloatScales() const { return FloatScaleUseCount > 0; }
	bool HasAnyTransformScales() const { return TransformScaleUseCount > 0; }

private:

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:
	
	using FPropertyScaledInstanceKey = TTuple<UE::MovieScene::FRootInstanceHandle, FMovieSceneSequenceID>;

	using FPropertyScaleEntityIDs = TArray<UE::MovieScene::FMovieSceneEntityID, TInlineAllocator<2>>;
	TMap<FPropertyScaledInstanceKey, FPropertyScaleEntityIDs> PropertyScaledInstances;

	int32 FloatScaleUseCount = 0;
	int32 TransformScaleUseCount = 0;
};

UCLASS(MinimalAPI)
class UTemplateSequencePropertyScalingEvaluatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UTemplateSequencePropertyScalingEvaluatorSystem(const FObjectInitializer& ObjInit);

	using FPropertyScaleKey = TTuple<UE::MovieScene::FRootInstanceHandle, FMovieSceneSequenceID, FGuid, FName>;
	using FPropertyScaleValue = TTuple<ETemplateSectionPropertyScaleType, float>;

	using FPropertyScaleValueArray = TArray<FPropertyScaleValue, TInlineAllocator<2>>;

	void ResetPropertyScales();
	void AddPropertyScale(const FPropertyScaleKey& Key, const FPropertyScaleValue& Value);
	void FindPropertyScales(const FPropertyScaleKey& Key, FPropertyScaleValueArray& OutValues) const;

private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override final;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;
	
private:

	struct FMultiPropertyScaleValue
	{
		FPropertyScaleValueArray Values;
	};

	TMap<FPropertyScaleKey, FMultiPropertyScaleValue> PropertyScales;
};

