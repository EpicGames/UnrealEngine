// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateSequence.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/World.h"
#include "Misc/PackageName.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieScenePossessable.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "MovieSceneSpawnable.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "UObject/AssetRegistryTagsContext.h"

#include "TemplateSequenceActor.h"
#include "TemplateSequencePlayer.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateSequence)

DEFINE_LOG_CATEGORY(LogTemplateSequence);

UTemplateSequence::UTemplateSequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MovieScene(nullptr)
{
	bParentContextsAreSignificant = true;
}

void UTemplateSequence::Initialize()
{
	MovieScene = NewObject<UMovieScene>(this, NAME_None, RF_Transactional);

	MovieScene->SetEvaluationType(EMovieSceneEvaluationType::WithSubFrames);

	FFrameRate TickResolution(24000, 1);
	MovieScene->SetTickResolutionDirectly(TickResolution);

	FFrameRate DisplayRate(30, 1);
	MovieScene->SetDisplayRate(DisplayRate);
}

FGuid UTemplateSequence::GetRootObjectBindingID() const
{
	if (MovieScene != nullptr && MovieScene->GetSpawnableCount() > 0)
	{
		const FMovieSceneSpawnable& FirstSpawnable = MovieScene->GetSpawnable(0);
		return FirstSpawnable.GetGuid();
	}

	return FGuid();
}

const UObject* UTemplateSequence::GetRootObjectSpawnableTemplate() const
{
	if (MovieScene != nullptr && MovieScene->GetSpawnableCount() > 0)
	{
		const FMovieSceneSpawnable& FirstSpawnable = MovieScene->GetSpawnable(0);
		return FirstSpawnable.GetObjectTemplate();
	}
	return nullptr;
}

void UTemplateSequence::BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context)
{
	if (UActorComponent* Component = Cast<UActorComponent>(&PossessedObject))
	{
		const FName ComponentName = Component->GetFName();
		BoundActorComponents.Add(ObjectId, ComponentName);
	}
}

bool UTemplateSequence::CanPossessObject(UObject& Object, UObject* InPlaybackContext) const
{
	UClass* ExpectedActorClass = BoundActorClass.Get();
	if (!ExpectedActorClass)
	{
		return false;
	}

	UActorComponent* Component = Cast<UActorComponent>(&Object);
	if (!Component)
	{
		return false;
	}

	AActor* Owner = Component->GetOwner();
	if (!Owner || !Owner->IsA(ExpectedActorClass))
	{
		return false;
	}

	return true;
}

void UTemplateSequence::LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	AActor* Actor = Cast<AActor>(Context);
	if (Actor == nullptr)
	{
		return;
	}

	const FName* ComponentName = BoundActorComponents.Find(ObjectId);
	if (ComponentName == nullptr)
	{
		return;
	}

	if (UActorComponent* FoundComponent = FindObject<UActorComponent>(Actor, *ComponentName->ToString(), EFindObjectFlags::None))
	{
		OutObjects.Add(FoundComponent);
	}
}

UMovieScene* UTemplateSequence::GetMovieScene() const
{
	return MovieScene;
}

UObject* UTemplateSequence::GetParentObject(UObject* Object) const
{
	if (UActorComponent* Component = Cast<UActorComponent>(Object))
	{
		return Component->GetOwner();
	}

	return nullptr;
}

void UTemplateSequence::UnbindPossessableObjects(const FGuid& ObjectId)
{
	BoundActorComponents.Remove(ObjectId);
}

void UTemplateSequence::UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context)
{
	BoundActorComponents.Remove(ObjectId);
}

void UTemplateSequence::UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context)
{
	BoundActorComponents.Remove(ObjectId);
}

FGuid UTemplateSequence::FindOrAddBinding(UObject* InObject)
{
	UObject* ParentObject = GetParentObject(InObject);
	FGuid    ParentGuid   = ParentObject ? FindOrAddBinding(ParentObject) : FGuid();

	if (ParentObject && !ParentGuid.IsValid())
	{
		UE_LOGF(LogTemplateSequence, Error, "Unable to possess object '%ls' because it's parent could not be bound.", *InObject->GetName());
		return FGuid();
	}

	// Perform a potentially slow lookup of every possessable binding in the sequence to see if we already have this
	UWorld* World = InObject->GetWorld();
	ensure(World);
	{
		ATemplateSequenceActor* OutActor = nullptr;
		UTemplateSequencePlayer* Player = UTemplateSequencePlayer::CreateTemplateSequencePlayer(World, this, FMovieSceneSequencePlaybackSettings(), OutActor);
		ensure(Player);
		
		// Don't override the default, the default assignment should remain bound so that objects will spawn
		OutActor->BindingOverride.bOverridesDefault = false;

		Player->Initialize(this, World, FMovieSceneSequencePlaybackSettings());
		Player->GetEvaluationState()->AssignSequence(MovieSceneSequenceID::Root, *this, *Player);
		Player->PlayTo(FMovieSceneSequencePlaybackParams(MovieScene->GetPlaybackRange().GetLowerBoundValue(), EUpdatePositionMethod::Play), FMovieSceneSequencePlayToParams());

		FGuid FoundBinding;
		for (int32 BindingIndex = 0; !FoundBinding.IsValid() && BindingIndex < ((const UMovieScene*)MovieScene)->GetBindings().Num(); ++BindingIndex)
		{
			for (TWeakObjectPtr<UObject> BoundObject : Player->FindBoundObjects(((const UMovieScene*)MovieScene)->GetBindings()[BindingIndex].GetObjectGuid(), MovieSceneSequenceID::Root))
			{
				if (BoundObject.IsValid() && BoundObject.Get()->GetClass() == InObject->GetClass())
				{
					if (!ParentObject || BoundObject.Get()->GetName() == InObject->GetName())
					{
						FoundBinding = ((const UMovieScene*)MovieScene)->GetBindings()[BindingIndex].GetObjectGuid();
						break;
					}
				}
			}
		}
		
		if (!FoundBinding.IsValid())
		{
			FGuid ExistingID = Player->FindObjectId(*InObject, MovieSceneSequenceID::Root);
			if (ExistingID.IsValid())
			{
				FoundBinding = ExistingID;
			}
		}

		Player->Stop();
		if (OutActor)
		{
			World->DestroyActor(OutActor);
		}

		if (FoundBinding.IsValid())
		{
			return FoundBinding;
		}
	}

	const FGuid NewGuid = MovieScene->AddPossessable(InObject->GetName(), InObject->GetClass());
	
	// Set up parent/child guids for possessables within spawnables
	if (ParentGuid.IsValid())
	{
		FMovieScenePossessable* ChildPossessable = MovieScene->FindPossessable(NewGuid);
		if (ensure(ChildPossessable))
		{
			ChildPossessable->SetParent(ParentGuid, MovieScene);
		}

		FMovieSceneSpawnable* ParentSpawnable = MovieScene->FindSpawnable(ParentGuid);
		if (ParentSpawnable)
		{
			ParentSpawnable->AddChildPossessable(NewGuid);
		}
	}

	BindPossessableObject(NewGuid, *InObject, ParentObject);

	return NewGuid;
}

FGuid UTemplateSequence::CreatePossessable(UObject* ObjectToPossess)
{
	return FindOrAddBinding(ObjectToPossess);
}

bool UTemplateSequence::AllowsSpawnableObjects() const
{
	return true;
}

UObject* UTemplateSequence::MakeSpawnableTemplateFromInstance(UObject& InSourceObject, FName ObjectName)
{
	return MovieSceneHelpers::MakeSpawnableTemplateFromInstance(InSourceObject, MovieScene, ObjectName);
}

void UTemplateSequence::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UTemplateSequence::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (BoundActorClass.IsValid())
	{
		FAssetRegistryTag Tag("BoundActorClass", BoundActorClass->GetPathName(), FAssetRegistryTag::TT_Alphabetical);
		Context.AddTag(Tag);
	}
	else
	{
		Context.AddTag(FAssetRegistryTag("BoundActorClass", "(None)", FAssetRegistryTag::TT_Alphabetical));
	}
}

#if WITH_EDITOR

void UTemplateSequence::ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const
{
	Super::ThreadedPostLoadAssetRegistryTagsOverride(Context);

	static const FName BoundActorClassTagName(TEXT("BoundActorClass"));
	FString BoundActorClassTagValue = Context.GetAssetData().GetTagValueRef<FString>(BoundActorClassTagName);
	if (!BoundActorClassTagValue.IsEmpty() && FPackageName::IsShortPackageName(BoundActorClassTagValue))
	{
		FTopLevelAssetPath BoundActorClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(BoundActorClassTagValue, ELogVerbosity::Warning, TEXT("UTemplateSequence::ThreadedPostLoadAssetRegistryTagsOverride"));
		if (!BoundActorClassPathName.IsNull())
		{
			Context.AddTagToUpdate(FAssetRegistryTag(BoundActorClassTagName, BoundActorClassPathName.ToString(), FAssetRegistryTag::TT_Alphabetical));
		}
	}
}

FText UTemplateSequence::GetDisplayName() const
{
	return UMovieSceneSequence::GetDisplayName();
}

ETrackSupport UTemplateSequence::IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const
{
	if (InTrackClass == UMovieSceneSkeletalAnimationTrack::StaticClass() ||
		InTrackClass == UMovieSceneSpawnTrack::StaticClass())
	{
		return ETrackSupport::Supported;
	}

	return Super::IsTrackSupportedImpl(InTrackClass);
}

void UTemplateSequence::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	OutMetadata.Add(
		"BoundActorClass",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("TemplateSequence", "BoundActorClass_Label", "Bound Actor Class"))
			.SetTooltip(NSLOCTEXT("TemplateSequence", "BoundActorClass_Tooltip", "The type of actor bound to this template sequence"))
		);
}

#endif

