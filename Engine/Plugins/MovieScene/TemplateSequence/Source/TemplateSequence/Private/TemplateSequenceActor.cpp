// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateSequenceActor.h"
#include "TemplateSequence.h"
#include "Engine/World.h"
#include "Net/UnrealNetwork.h"
#include "TemplateSequencePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateSequenceActor)

ATemplateSequenceActor::ATemplateSequenceActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;

	SequencePlayer = ObjectInitializer.CreateDefaultSubobject<UTemplateSequencePlayer>(this, "AnimationPlayer");

	// Note: we don't set `bCanEverTick` because we add this actor to the list of level sequence actors in the world.
	//PrimaryActorTick.bCanEverTick = false;
}

void ATemplateSequenceActor::PostInitProperties()
{
	Super::PostInitProperties();

	// Have to initialize this here as any properties set on default subobjects inside the constructor
	// Get stomped by the CDO's properties when the constructor exits.
	SequencePlayer->SetPlaybackClient(this);
}

void ATemplateSequenceActor::PostLoad()
{
	Super::PostLoad();

}

void ATemplateSequenceActor::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ATemplateSequenceActor, SequencePlayer);
}

UTemplateSequencePlayer* ATemplateSequenceActor::GetSequencePlayer() const
{
	return SequencePlayer && SequencePlayer->GetSequence() ? SequencePlayer : nullptr;
}

void ATemplateSequenceActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	SequencePlayer->InitializeForTick(this);

	InitializePlayer();
}

void ATemplateSequenceActor::BeginPlay()
{
	Super::BeginPlay();
	
	if (PlaybackSettings.bAutoPlay)
	{
		SequencePlayer->Play();
	}
}

void ATemplateSequenceActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (SequencePlayer)
	{
		// See comment in LevelSequenceActor.cpp
		SequencePlayer->Stop();
		SequencePlayer->TearDown();
	}

	Super::EndPlay(EndPlayReason);
}

UTemplateSequence* ATemplateSequenceActor::GetSequence() const
{
	return Cast<UTemplateSequence>(TemplateSequence.ResolveObject());
}

UTemplateSequence* ATemplateSequenceActor::LoadSequence() const
{
	return Cast<UTemplateSequence>(TemplateSequence.TryLoad());
}

void ATemplateSequenceActor::SetSequence(UTemplateSequence* InSequence)
{
	if (!SequencePlayer->IsPlaying())
	{
		TemplateSequence = InSequence;

		if (InSequence)
		{
			SequencePlayer->Initialize(InSequence, GetWorld(), PlaybackSettings);
		}
	}
}

void ATemplateSequenceActor::InitializePlayer()
{
	if (TemplateSequence.IsValid() && GetWorld()->IsGameWorld())
	{
		// Attempt to resolve the asset without loading it
		UTemplateSequence* TemplateSequenceAsset = GetSequence();
		if (TemplateSequenceAsset)
		{
			// Level sequence is already loaded. Initialize the player if it's not already initialized with this sequence
			if (TemplateSequenceAsset != SequencePlayer->GetSequence())
			{
				SequencePlayer->Initialize(TemplateSequenceAsset, GetWorld(), PlaybackSettings);
			}
		}
		else if (!IsAsyncLoading())
		{
			TemplateSequenceAsset = LoadSequence();
			if (TemplateSequenceAsset != SequencePlayer->GetSequence())
			{
				SequencePlayer->Initialize(TemplateSequenceAsset, GetWorld(), PlaybackSettings);
			}
		}
		else
		{
			LoadPackageAsync(TemplateSequence.GetLongPackageName(), FLoadPackageAsyncDelegate::CreateUObject(this, &ATemplateSequenceActor::OnSequenceLoaded));
		}
	}
}

void ATemplateSequenceActor::OnSequenceLoaded(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
{
	if (Result == EAsyncLoadingResult::Succeeded)
	{
		UTemplateSequence* TemplateSequenceAsset = GetSequence();
		if (SequencePlayer && SequencePlayer->GetSequence() != TemplateSequenceAsset)
		{
			SequencePlayer->Initialize(TemplateSequenceAsset, GetWorld(), PlaybackSettings);
		}
	}
}

bool ATemplateSequenceActor::RetrieveBindingOverrides(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	// If the given object binding ID corresponds to the template sequence's root binding, return the override we have.
	// Otherwise, let the runtime do the normal lookup.
	UTemplateSequence* TemplateSequenceObject = GetSequence();
	if (!TemplateSequenceObject)
	{
		return true;
	}

	const FGuid RootBindingID = TemplateSequenceObject->GetRootObjectBindingID();
	if (!RootBindingID.IsValid())
	{
		return true;
	}
	if (RootBindingID != InBindingId)
	{
		return true;
	}

	if (UObject* Object = BindingOverride.Object.Get())
	{
		OutObjects.Add(Object);
	}

	return !BindingOverride.bOverridesDefault;
}

UObject* ATemplateSequenceActor::GetInstanceData() const
{
	return nullptr;
}

void ATemplateSequenceActor::SetBinding(AActor* Actor, bool bOverridesDefault)
{
	BindingOverride.Object = Actor;
	BindingOverride.bOverridesDefault = bOverridesDefault;

	// Invalidate the root bound object's mapping, if any.
	UTemplateSequence* TemplateSequenceObject = GetSequence();
	if (SequencePlayer && TemplateSequenceObject)
	{
		const FGuid RootBindingID = TemplateSequenceObject->GetRootObjectBindingID();
		if (RootBindingID.IsValid())
		{
			SequencePlayer->GetEvaluationState()->Invalidate(RootBindingID, MovieSceneSequenceID::Root);
		}
	}
}

#if WITH_EDITOR

bool ATemplateSequenceActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	UTemplateSequence* TemplateSequenceAsset = LoadSequence();

	if (TemplateSequenceAsset)
	{
		Objects.Add(TemplateSequenceAsset);
	}

	Super::GetReferencedContentObjects(Objects);

	return true;
}

#endif // WITH_EDITOR

