// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateSequenceComponentTypes.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

namespace UE
{
namespace MovieScene
{

FTemplateSequenceComponentTypes* FTemplateSequenceComponentTypes::Get()
{
	static TUniquePtr<FTemplateSequenceComponentTypes> GTemplateSequenceComponentTypes;
	if (!GTemplateSequenceComponentTypes.IsValid())
	{
		GTemplateSequenceComponentTypes.Reset(new FTemplateSequenceComponentTypes);
	}
	return GTemplateSequenceComponentTypes.Get();
}

FTemplateSequenceComponentTypes::FTemplateSequenceComponentTypes()
{
	using namespace UE::MovieScene;

	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

	ComponentRegistry->NewComponentType(&TemplateSequence, TEXT("Template Sequence"));
	ComponentRegistry->NewComponentType(&PropertyScale, TEXT("Template Sequence Property Scale"));
	ComponentRegistry->NewComponentType(&PropertyScaleReverseBindingLookup, TEXT("Property Scale Reverse Binding Lookup"));

	ComponentRegistry->Factories.DuplicateChildComponent(TemplateSequence);
	ComponentRegistry->Factories.DuplicateChildComponent(PropertyScale);

	Tags.IsPropertyScaled = ComponentRegistry->NewTag(TEXT("Is Property Scaled"));
}

} // namespace MovieScene
} // namespace UE

