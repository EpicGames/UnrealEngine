// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "CameraAnimationSequencePlayer.h"

class FTemplateSequenceModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		UCameraAnimationSequenceCameraStandIn::RegisterCameraStandIn();
	}

	virtual void ShutdownModule() override
	{
		UCameraAnimationSequenceCameraStandIn::UnregisterCameraStandIn();
	}
};

IMPLEMENT_MODULE(FTemplateSequenceModule, TemplateSequence);
