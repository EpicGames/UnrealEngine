// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateSequencePlayer.h"
#include "TemplateSequence.h"
#include "TemplateSequenceActor.h"
#include "TemplateSequenceSpawnRegister.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateSequencePlayer)

UTemplateSequencePlayer::UTemplateSequencePlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UTemplateSequencePlayer* UTemplateSequencePlayer::CreateTemplateSequencePlayer(UObject* WorldContextObject, UTemplateSequence* TemplateSequence, FMovieSceneSequencePlaybackSettings Settings, ATemplateSequenceActor*& OutActor)
{
	if (TemplateSequence == nullptr)
	{
		return nullptr;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World == nullptr)
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.ObjectFlags |= RF_Transient;
	SpawnParams.bAllowDuringConstructionScript = true;

	// Defer construction for autoplay so that BeginPlay() is called
	SpawnParams.bDeferConstruction = true;

	ATemplateSequenceActor* Actor = World->SpawnActor<ATemplateSequenceActor>(SpawnParams);

	Actor->PlaybackSettings = Settings;
	Actor->TemplateSequence = TemplateSequence;

	Actor->InitializePlayer();
	OutActor = Actor;

	FTransform DefaultTransform;
	Actor->FinishSpawning(DefaultTransform);

	return Actor->SequencePlayer;
}

void UTemplateSequencePlayer::Initialize(UMovieSceneSequence* InSequence, UWorld* InWorld, const FMovieSceneSequencePlaybackSettings& InSettings)
{
	SpawnRegister = MakeShareable(new FTemplateSequenceSpawnRegister());

	World = InWorld;

	Super::Initialize(InSequence, InSettings);
}

UObject* UTemplateSequencePlayer::GetPlaybackContext() const
{
	return World.Get();
}

