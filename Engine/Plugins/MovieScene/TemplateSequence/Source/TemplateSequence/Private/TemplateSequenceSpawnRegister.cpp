// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateSequenceSpawnRegister.h"

FTemplateSequenceSpawnRegister::FTemplateSequenceSpawnRegister()
{
}
