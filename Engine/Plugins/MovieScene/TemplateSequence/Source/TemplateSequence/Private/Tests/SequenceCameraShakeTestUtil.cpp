// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceCameraShakeTestUtil.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequenceCameraShakeTestUtil)

FMinimalViewInfo USequenceCameraShakeTestUtil::GetCameraCachePOV(APlayerController* PlayerController)
{
	return PlayerController->PlayerCameraManager->GetCameraCacheView();
}

FMinimalViewInfo USequenceCameraShakeTestUtil::GetLastFrameCameraCachePOV(APlayerController* PlayerController)
{
	return PlayerController->PlayerCameraManager->GetLastFrameCameraCacheView();
}

bool USequenceCameraShakeTestUtil::GetPostProcessBlendCache(APlayerController* PlayerController, int32 PPIndex, FPostProcessSettings& OutPPSettings, float& OutPPBlendWeight)
{
	const TArray<FPostProcessSettings>* PPSettings;
	const TArray<float>* PPBlendWeights;
	PlayerController->PlayerCameraManager->GetCachedPostProcessBlends(PPSettings, PPBlendWeights);
	if (ensure(PPSettings && PPSettings->Num() > PPIndex && PPBlendWeights && PPBlendWeights->Num() > PPIndex))
	{
		OutPPSettings = (*PPSettings)[0];
		OutPPBlendWeight = (*PPBlendWeights)[0];
		return true;
	}
	return false;
}

