// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "SequenceCameraShakeTestUtil.generated.h"

struct FMinimalViewInfo;
struct FPostProcessSettings;

UCLASS(BlueprintType)
class USequenceCameraShakeTestUtil : public UBlueprintFunctionLibrary
{
public:
	GENERATED_BODY()

	UFUNCTION(BlueprintPure, Category="Test")
	static FMinimalViewInfo GetCameraCachePOV(APlayerController* PlayerController);

	UFUNCTION(BlueprintPure, Category="Test")
	static FMinimalViewInfo GetLastFrameCameraCachePOV(APlayerController* PlayerController);

	UFUNCTION(BlueprintPure, Category="Test")
	static bool GetPostProcessBlendCache(APlayerController* PlayerController, int32 PPIndex, FPostProcessSettings& OutPPSettings, float& OutPPBlendWeight);
};

