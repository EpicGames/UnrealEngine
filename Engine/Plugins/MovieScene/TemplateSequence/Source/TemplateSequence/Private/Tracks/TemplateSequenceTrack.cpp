// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/TemplateSequenceTrack.h"
#include "MovieScene.h"
#include "TemplateSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "Sections/TemplateSequenceSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateSequenceTrack)

#define LOCTEXT_NAMESPACE "TemplateSequenceTrack"

UTemplateSequenceTrack::UTemplateSequenceTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
}

bool UTemplateSequenceTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UTemplateSequenceSection::StaticClass();
}

UMovieSceneSection* UTemplateSequenceTrack::CreateNewSection()
{
	return NewObject<UTemplateSequenceSection>(this, NAME_None, RF_Transactional);
}

UMovieSceneSection* UTemplateSequenceTrack::AddNewTemplateSequenceSection(FFrameNumber KeyTime, UTemplateSequence* InSequence)
{
	UTemplateSequenceSection* NewSection = Cast<UTemplateSequenceSection>(CreateNewSection());
	{
		UMovieScene* OuterMovieScene = GetTypedOuter<UMovieScene>();
		UMovieScene* InnerMovieScene = InSequence->GetMovieScene();

		int32      InnerSequenceLength = UE::MovieScene::DiscreteSize(InnerMovieScene->GetPlaybackRange());
		FFrameTime OuterSequenceLength = ConvertFrameTime(InnerSequenceLength, InnerMovieScene->GetTickResolution(), OuterMovieScene->GetTickResolution());

		NewSection->InitialPlacement(Sections, KeyTime, OuterSequenceLength.FrameNumber.Value, SupportsMultipleRows());
		NewSection->SetSequence(InSequence);
	}

	AddSection(*NewSection);

	return NewSection;
}

#if WITH_EDITORONLY_DATA

FText UTemplateSequenceTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Template Animation");
}

#endif

#undef LOCTEXT_NAMESPACE

