// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TemplateSequence.h"
#include "CameraAnimationSequence.generated.h"

#define UE_API TEMPLATESEQUENCE_API

/*
 * A template sequence specifically designed for playing on cameras.
 */
UCLASS(MinimalAPI, BlueprintType)
class UCameraAnimationSequence : public UTemplateSequence
{
public:
	GENERATED_BODY()

	UE_API UCameraAnimationSequence(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	UE_API virtual FText GetDisplayName() const override;
#endif
};

#undef UE_API
