// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CineCameraSettings.h"
#include "TemplateSequencePlayer.h"
#include "CameraAnimationSequencePlayer.generated.h"

#define UE_API TEMPLATESEQUENCE_API

struct FMinimalViewInfo;

/**
 * A dummy class that we give to a sequence in lieu of an actual camera actor.
 */
UCLASS(MinimalAPI)
class UCameraAnimationSequenceCameraStandIn : public UObject
{
public:

	GENERATED_BODY()

	UE_API UCameraAnimationSequenceCameraStandIn(const FObjectInitializer& ObjInit);

public:
	/**
	 * Scene component properties
	 *
	 * Transform doesn't need to be a UPROPERTY because we register a custom getter/setter. This is
	 * because the sequence runtime expects that in order to use the intermediate transform struct.
	 */
	const FTransform& GetTransform() const { return Transform; }
	void SetTransform(const FTransform& InTransform) { Transform = InTransform; }

	/** Camera component properties */
	UPROPERTY()
	float FieldOfView;

	UPROPERTY()
	uint8 bConstrainAspectRatio : 1;

	UPROPERTY()
	float AspectRatio;

	UPROPERTY()
	FPostProcessSettings PostProcessSettings;

	UPROPERTY()
	float PostProcessBlendWeight;

	/** Cine camera component properties */
	UPROPERTY()
	FCameraFilmbackSettings Filmback;

	UPROPERTY()
	FCameraLensSettings LensSettings;

	UPROPERTY()
	FCameraFocusSettings FocusSettings;

	UPROPERTY()
	float CurrentFocalLength;

	UPROPERTY()
	float CurrentAperture;

	UPROPERTY()
	float CurrentFocusDistance;

public:
	/** Initialize this object's properties based on the given sequence's root object template */
	UE_API void Initialize(UTemplateSequence* TemplateSequence);

	/** Reset the properties of the stand-in every frame before animation */
	UE_API void Reset(const FMinimalViewInfo& ViewInfo, UMovieSceneEntitySystemLinker* Linker);

	/** Recompute camera and lens settings after each frame */
	UE_API void RecalcDerivedData();

private:
	UE_API void ResetDefaultValues(const FMinimalViewInfo& ViewInfo);
	UE_API void UpdateInitialPropertyValues(UMovieSceneEntitySystemLinker* Linker);

	/** Register the stand-in class with the sequencer ECS component registry */
	static UE_API void RegisterCameraStandIn();
	/** Unregister the stand-in class with the sequencer ECS component registry */
	static UE_API void UnregisterCameraStandIn();

private:
	static UE_API bool bRegistered;

	FTransform Transform;

	bool bIsCineCamera = false;
	float WorldToMeters = 0.f;

	friend class FTemplateSequenceModule;
};

/**
 * A lightweight sequence player for playing camera animation sequences.
 */
UCLASS(MinimalAPI)
class UCameraAnimationSequencePlayer
	: public UObject
	, public IMovieScenePlayer
{
public:
	GENERATED_BODY()

	UE_API UCameraAnimationSequencePlayer(const FObjectInitializer& ObjInit);
	UE_API virtual ~UCameraAnimationSequencePlayer();

	/**
	 * Initializes this player with the given sequence
	 *
	 * @param InSequence    The sequence to play
	 * @param StartOffset   The offset to start at, in frames (display rate)
	 * @param DurationOverride  A duration to use instead of the natural duration of the sequence
	 */
	UE_API void Initialize(UMovieSceneSequence* InSequence, int32 StartOffset = 0, float DurationOverride = 0.f);

	/**
	 * Returns whether this player is still valid. It could be forcibly torn-down if the camera
	 * animation subsystem was destroyed with the world.
	 */
	UE_API bool IsValid() const;

	/**
	 * Start playing the sequence
	 *
	 * @param bLoop              Whether to loop playback
	 * @param bRandomStartTime   Whether to start at a random time inside the playback range
	 * 
	 * Note that if StartOffset was set, the random start time will be chosen within
	 * the reduced (offset) playback range.
	 */
	UE_API void Play(bool bLoop = false, bool bRandomStartTime = false);

	/**
	 * Advance play to the given time
	 *
	 * @param NewPosition   The time to advance to, in ticks
	 */
	UE_API void Update(FFrameTime NewPosition);

	/**
	 * Jumps to the given time, in ticks
	 *
	 * @param New Position   The time to jump to, in ticks
	 */
	UE_API void Jump(FFrameTime NewPosition);

	/** Stop playing the sequence */
	UE_API void Stop();

	/** Gets whether playback is looping */
	bool GetIsLooping() const { return bIsLooping; }
	/** Get the sequence display rate */
	FFrameRate GetInputRate() const { return PlayPosition.GetInputRate(); }
	/** Get the sequence tick resolution */
	FFrameRate GetOutputRate() const { return PlayPosition.GetOutputRate(); }
	/** Get the start frame of the current sequence */
	FFrameNumber GetStartFrame() const { return StartFrame; }
	/** Get the duration of the current sequence in frames (display rate) */
	UE_API FFrameTime GetDuration() const;
	/** Gets the current play position in frames (display rate) */
	UE_API FFrameTime GetCurrentPosition() const;

	/** Sets an object that can be used to bind everything in the sequence */
	UE_API void SetBoundObjectOverride(UObject* InObject);

public:

	// IMovieScenePlayer interface
	virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override { return RootTemplateInstance; }
	UE_API virtual UMovieSceneEntitySystemLinker* ConstructEntitySystemLinker() override;
	UE_API virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override;
	virtual UObject* AsUObject() override { return this; }
	virtual FMovieSceneSpawnRegister& GetSpawnRegister() override { return SpawnRegister; }

	virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override {}
	virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) override {}
	virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const override {}
	virtual bool CanUpdateCameraCut() const override { return false; }
	virtual void UpdateCameraCut(UObject* CameraObject, const EMovieSceneCameraCutParams& CameraCutParams) override {}
	
	// UObject interface 
	UE_API virtual void BeginDestroy() override;

protected:

	UE_API virtual void ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& Sequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;

	UE_API void OnAbandonLinker(UMovieSceneEntitySystemLinker* InLinker);

private:

	FSequenceCameraShakeSpawnRegister SpawnRegister;

	/** Bound object overrides */
	UPROPERTY(transient)
	TObjectPtr<UObject> BoundObjectOverride;

	/** The sequence to play back */
	UPROPERTY(transient)
	TObjectPtr<UMovieSceneSequence> Sequence;

	/** The evaluation template instance */
	UPROPERTY(transient)
	FMovieSceneRootEvaluationTemplateInstance RootTemplateInstance;

	/** Play position helper */
	FMovieScenePlaybackPosition PlayPosition;

	/** Start frame for the sequence */
	FFrameNumber StartFrame;

	/** The sequence duration in frames */
	FFrameTime DurationFrames;

	/** The total duration we need to play */
	FFrameTime TotalDurationFrames;

	/** Accumulated number of loops played so far */
	uint16 LoopsPlayed = 0; 

	/** Whether we should be looping */
	bool bIsLooping = false;

	/** Whether we need to loop due to a duration override */
	bool bDurationRequiresLooping = false;

	/** Movie player status. */
	TEnumAsByte<EMovieScenePlayerStatus::Type> Status;
};

#undef UE_API
