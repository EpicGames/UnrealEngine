// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "Subsystems/WorldSubsystem.h"
#include "CameraAnimationSequenceSubsystem.generated.h"

class UMovieSceneEntitySystemLinker;
class FMovieSceneEntitySystemRunner;

UCLASS()
class UCameraAnimationSpawnableSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	UCameraAnimationSpawnableSystem(const FObjectInitializer& ObjInit);

	void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents);

private:
	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const;
};

UCLASS()
class UCameraAnimationBoundObjectInstantiator : public UMovieSceneEntityInstantiatorSystem
{
	GENERATED_BODY()

public:
	UCameraAnimationBoundObjectInstantiator(const FObjectInitializer& ObjInit);

	void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents);

private:
	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const;
};

UCLASS()
class UCameraAnimationEntitySystemLinker : public UMovieSceneEntitySystemLinker
{
	GENERATED_BODY()

public:
	UCameraAnimationEntitySystemLinker(const FObjectInitializer& ObjInit);
};

/**
 * World subsystem that holds global objects for handling camera animation sequences.
 */
UCLASS()
class UCameraAnimationSequenceSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	/** Get the camera animation sequence subsystem for the given world */
	static UCameraAnimationSequenceSubsystem* GetCameraAnimationSequenceSubsystem(const UWorld* InWorld);

	/** Create a new linker setup for handling camera animation sequences */
	static UMovieSceneEntitySystemLinker* CreateLinker(UObject* Outer, FName Name);

	/** Gets the system category for camera animation specific systems */
	static UE::MovieScene::EEntitySystemCategory GetCameraAnimationSystemCategory();

	/** Construct a new camera animation sequence subsystem */
	UCameraAnimationSequenceSubsystem();
	/** Destroy the camera animation sequence subsystem */
	virtual ~UCameraAnimationSequenceSubsystem();

	/** Gets the Sequencer linker owned by this sybsystem */
	UMovieSceneEntitySystemLinker* GetLinker(bool bAutoCreate = true);

	/** Get's this subsystem's camera anim runner */
	TSharedPtr<FMovieSceneEntitySystemRunner> GetRunner() const;

private:
	// Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	// End USubsystem
	
private:
	/** The global Sequencer linker that contains all the shakes and camera animations */
	UPROPERTY(transient)
	TObjectPtr<UMovieSceneEntitySystemLinker> Linker;

	/** The global Sequencer runner that runs all the shakes and camera animations */
	TSharedPtr<FMovieSceneEntitySystemRunner> Runner;
};
