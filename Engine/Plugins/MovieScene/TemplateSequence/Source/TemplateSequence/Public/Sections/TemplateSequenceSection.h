// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "Sections/MovieSceneSubSection.h"
#include "TemplateSequenceSection.generated.h"

#define UE_API TEMPLATESEQUENCE_API

enum class EMovieSceneChannelProxyType : uint8;

/**
 * Defines the type of property scaling for a template sequence.
 */
UENUM()
enum class ETemplateSectionPropertyScaleType
{
	/** Scales a float property */
	FloatProperty,
	/** Scales the location channels (X, Y, Z) of a transform property */
	TransformPropertyLocationOnly,
	/** Scales the rotation channels (yaw, pitch, roll) of a transform property */
	TransformPropertyRotationOnly
};

/**
 * Defines a property scaling for a template sequence.
 */
USTRUCT()
struct FTemplateSectionPropertyScale
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ObjectBinding;

	UPROPERTY()
	FMovieScenePropertyBinding PropertyBinding;

	UPROPERTY()
	ETemplateSectionPropertyScaleType PropertyScaleType= ETemplateSectionPropertyScaleType::FloatProperty;

	UPROPERTY()
	FMovieSceneFloatChannel FloatChannel;
};

/**
 * Defines the section for a template sequence track.
 */
UCLASS(MinimalAPI)
class UTemplateSequenceSection 
	: public UMovieSceneSubSection
{
public:

	GENERATED_BODY()

	UE_API UTemplateSequenceSection(const FObjectInitializer& ObjInitializer);

	UE_API void AddPropertyScale(const FTemplateSectionPropertyScale& InPropertyScale);
	UE_API void RemovePropertyScale(int32 InIndex);

	// UMovieSceneSection interface
	UE_API virtual bool ShowCurveForChannel(const void *Channel) const override;
	UE_API virtual void OnDilated(float DilationFactor, FFrameNumber Origin) override;

protected:

	UE_API virtual EMovieSceneChannelProxyType CacheChannelProxy() override;

	UE_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	UE_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity) override;

	/**~ UObject implementation */
	UE_API virtual void PostInitProperties() override;

public:

	UPROPERTY()
	TArray<FTemplateSectionPropertyScale> PropertyScales;
};

#undef UE_API
