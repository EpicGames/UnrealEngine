// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "SequenceCameraShake.generated.h"

#define UE_API TEMPLATESEQUENCE_API

struct FMinimalViewInfo;

class UCameraAnimationSequenceCameraStandIn;
class UMovieSceneEntitySystemLinker;
class UMovieSceneSequence;
class UCameraAnimationSequencePlayer;

/**
 * A camera shake pattern that plays a sequencer animation.
 */
UCLASS(MinimalAPI)
class USequenceCameraShakePattern : public UCameraShakePattern
{
public:

	GENERATED_BODY()

	UE_API USequenceCameraShakePattern(const FObjectInitializer& ObjInit);

public:

	/** Source camera animation sequence to play. */
	UPROPERTY(EditAnywhere, Category=CameraShake)
	TObjectPtr<class UCameraAnimationSequence> Sequence;

	/** Scalar defining how fast to play the anim. */
	UPROPERTY(EditAnywhere, Category=CameraShake, meta=(ClampMin="0.001"))
	float PlayRate;

	/** Scalar defining how "intense" to play the anim. */
	UPROPERTY(EditAnywhere, Category=CameraShake, meta=(ClampMin="0.0"))
	float Scale;

	/** Linear blend-in time. */
	UPROPERTY(EditAnywhere, Category=CameraShake, meta=(ClampMin="0.0"))
	float BlendInTime;

	/** Linear blend-out time. */
	UPROPERTY(EditAnywhere, Category=CameraShake, meta=(ClampMin="0.0"))
	float BlendOutTime;

	/** When bRandomSegment is true, defines how long the sequence should play. */
	UPROPERTY(EditAnywhere, Category=CameraShake, meta=(ClampMin="0.0", EditCondition="bRandomSegment"))
	float RandomSegmentDuration;

	/**
	 * When true, plays a random snippet of the sequence for RandomSegmentDuration seconds.
	 *
	 * @note The sequence we be forced to loop when bRandomSegment is enabled, in case the duration
	 *       is longer than what's left to play from the random start time.
	 */
	UPROPERTY(EditAnywhere, Category=CameraShake)
	bool bRandomSegment;

private:

	// UCameraShakeBase interface
	UE_API virtual void GetShakePatternInfoImpl(FCameraShakeInfo& OutInfo) const override;
	UE_API virtual void StartShakePatternImpl(const FCameraShakePatternStartParams& Params) override;
	UE_API virtual void UpdateShakePatternImpl(const FCameraShakePatternUpdateParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
	UE_API virtual void ScrubShakePatternImpl(const FCameraShakePatternScrubParams& Params, FCameraShakePatternUpdateResult& OutResult) override;
	UE_API virtual bool IsFinishedImpl() const override;
	UE_API virtual void StopShakePatternImpl(const FCameraShakePatternStopParams& Params) override;
	UE_API virtual void TeardownShakePatternImpl() override;

	void UpdateCamera(FFrameTime NewPosition, const FMinimalViewInfo& InPOV, FCameraShakePatternUpdateResult& OutResult);

private:

	/** The player we use to play the camera animation sequence */
	UPROPERTY(Instanced, Transient)
	TObjectPtr<UCameraAnimationSequencePlayer> Player;

	/** Standin for the camera actor and components */
	UPROPERTY(Instanced, Transient)
	TObjectPtr<UCameraAnimationSequenceCameraStandIn> CameraStandIn;

	/** State tracking */
	FCameraShakeState State;
};

#undef UE_API
