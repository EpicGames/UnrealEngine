// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequence.h"
#include "TemplateSequence.generated.h"

#define UE_API TEMPLATESEQUENCE_API

class MovieSceneTrack;
DECLARE_LOG_CATEGORY_EXTERN(LogTemplateSequence, Log, All);

/*
 * Movie scene animation that can be instanced multiple times inside a level sequence.
 */
UCLASS(MinimalAPI, BlueprintType)
class UTemplateSequence : public UMovieSceneSequence
{
public:
	GENERATED_BODY()

	UE_API UTemplateSequence(const FObjectInitializer& ObjectInitializer);

	UE_API void Initialize();

	/** Gets the object binding that corresponds to the root spawnable that serves as the template. */
	UE_API FGuid GetRootObjectBindingID() const;

	/** Gets the root spawnable object template. */
	UE_API const UObject* GetRootObjectSpawnableTemplate() const;

	//~ UMovieSceneSequence interface
	UE_API virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override;
	UE_API virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override;
	UE_API virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	UE_API virtual UMovieScene* GetMovieScene() const override;
	UE_API virtual UObject* GetParentObject(UObject* Object) const override;
	UE_API virtual void UnbindPossessableObjects(const FGuid& ObjectId) override;
	UE_API virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context) override;
	UE_API virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context) override;

	UE_API virtual FGuid CreatePossessable(UObject* ObjectToPossess) override;
	UE_API virtual bool AllowsSpawnableObjects() const override;

	UE_API virtual UObject* MakeSpawnableTemplateFromInstance(UObject& InSourceObject, FName ObjectName) override;

	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDisplayName() const override;

	UE_API virtual ETrackSupport IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const override;
	UE_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
protected:
	UE_API virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const;
#endif

private:
	
	FGuid FindOrAddBinding(UObject* ObjectToPossess);

public:

	UPROPERTY()
	TObjectPtr<UMovieScene> MovieScene;

	UPROPERTY()
	TSoftClassPtr<AActor> BoundActorClass;

	UPROPERTY()
	TMap<FGuid, FName> BoundActorComponents;
};

#undef UE_API
