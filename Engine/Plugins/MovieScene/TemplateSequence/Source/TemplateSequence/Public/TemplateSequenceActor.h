// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieScenePlaybackClient.h"
#include "GameFramework/Actor.h"
#include "MovieSceneSequencePlaybackSettings.h"
#include "TemplateSequenceActor.generated.h"

#define UE_API TEMPLATESEQUENCE_API

class UTemplateSequencePlayer;
namespace EEndPlayReason { enum Type : int; }

class UTemplateSequence;

/**
 * Template sequence binding override data
 *
 * This is similar to FMovieSceneBindingOverrideData, but works only for a template sequence's root object,
 * so we don't need it to store the object binding ID.
 */
USTRUCT(BlueprintType)
struct FTemplateSequenceBindingOverrideData
{
	GENERATED_BODY()

	FTemplateSequenceBindingOverrideData()
		: bOverridesDefault(true)
	{
	}

	/** Specifies the object binding to override. */
	UPROPERTY(EditAnywhere, Category = "Binding")
	TWeakObjectPtr<UObject> Object;

	/** Specifies whether the default assignment should remain bound (false) or if this should completely override the default binding (true). */
	UPROPERTY(EditAnywhere, Category = "Binding")
	bool bOverridesDefault;
};

/**
 * Actor responsible for controlling a specific template sequence in the world.
 */
UCLASS(MinimalAPI, hideCategories = (Rendering, Physics, LOD, Activation, Input))
class ATemplateSequenceActor
	: public AActor
	, public IMovieScenePlaybackClient
{
public:

	GENERATED_BODY()

	UE_API ATemplateSequenceActor(const FObjectInitializer& ObjectInitializer);

public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Playback", meta = (ShowOnlyInnerProperties))
	FMovieSceneSequencePlaybackSettings PlaybackSettings;

	UPROPERTY(Instanced, Transient, Replicated, BlueprintReadOnly, BlueprintGetter = GetSequencePlayer, Category = "Playback", meta = (ExposeFunctionCategories = "Sequencer|Player"))
	TObjectPtr<UTemplateSequencePlayer> SequencePlayer;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "General", meta = (AllowedClasses = "/Script/TemplateSequence.TemplateSequence"))
	FSoftObjectPath TemplateSequence;

	/** The override for the template sequence's root object binding. See SetBinding. */
	UPROPERTY(BlueprintReadOnly, Category = "General")
	FTemplateSequenceBindingOverrideData BindingOverride;

public:

	/**
	 * Get the template sequence being played by this actor.
	 *
	 * @return the template sequence, or nullptr if it is not assigned or cannot be loaded
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API UTemplateSequence* GetSequence() const;

	/**
	 * Get the template sequence being played by this actor.
	 *
	 * @return the template sequence, or nullptr if it is not assigned or cannot be loaded
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API UTemplateSequence* LoadSequence() const;

	/**
	 * Set the template sequence being played by this actor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void SetSequence(UTemplateSequence* InSequence);

	/**
	 * Get the actor's sequence player, or nullptr if it not yet initialized.
	 */
	UFUNCTION(BlueprintGetter)
	UE_API UTemplateSequencePlayer* GetSequencePlayer() const;

	/**
	 * Set the actor to play the template sequence onto, by setting up an override for the template
	 * sequence's root object binding.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	UE_API void SetBinding(AActor* Actor, bool bOverridesDefault = true);

protected:

	//~ Begin IMovieScenePlaybackClient interface
	UE_API virtual bool RetrieveBindingOverrides(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	UE_API virtual UObject* GetInstanceData() const override;
	//~ End IMovieScenePlaybackClient interface

	//~ Begin AActor interface
	UE_API virtual void PostInitializeComponents() override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End AActor interface

	//~ Begin UObject interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

public:

	UE_API void InitializePlayer();

#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif

private:

	void OnSequenceLoaded(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result);
};

#undef UE_API
