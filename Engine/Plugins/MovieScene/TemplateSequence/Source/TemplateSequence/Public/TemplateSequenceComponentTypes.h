// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"

enum class ETemplateSectionPropertyScaleType;

struct FMovieSceneFloatChannel;

namespace UE
{
namespace MovieScene
{

/** Component data for how a template sequence should remap its object bindings to that of a parent object binding */
struct FTemplateSequenceComponentData
{
	FMovieSceneEvaluationOperand InnerOperand;
};

/** Component data for what property to scale inside a template sequence */
struct FTemplateSequencePropertyScaleComponentData
{
	/** Sequence ID where the property track lives */
	FMovieSceneSequenceID SubSequenceID;
	/** The object binding ID of the object for which to scale the given property */
	FGuid ObjectBinding;
	/** The property to scale */
	FMovieScenePropertyBinding PropertyBinding;
	/** The type of property scaling to do */
	ETemplateSectionPropertyScaleType PropertyScaleType;
};

struct FTemplateSequenceComponentTypes
{
public:
	static FTemplateSequenceComponentTypes* Get();

	TComponentTypeID<FTemplateSequenceComponentData> TemplateSequence;
	TComponentTypeID<FTemplateSequencePropertyScaleComponentData> PropertyScale;
	TComponentTypeID<FGuid> PropertyScaleReverseBindingLookup;

	struct
	{
		FComponentTypeID IsPropertyScaled;
	} Tags;

private:
	FTemplateSequenceComponentTypes();
};

} // namespace MovieScene
} // namespace UE
