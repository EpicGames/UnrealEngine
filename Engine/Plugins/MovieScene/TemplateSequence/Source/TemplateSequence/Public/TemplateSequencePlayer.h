// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequencePlayer.h"
#include "TemplateSequencePlayer.generated.h"

#define UE_API TEMPLATESEQUENCE_API

struct FMovieSceneSequencePlaybackSettings;

class ATemplateSequenceActor;
class UTemplateSequence;
class UMovieSceneSpawnableBindingBase;

UCLASS(MinimalAPI, BlueprintType)
class UTemplateSequencePlayer : public UMovieSceneSequencePlayer
{
public:

	GENERATED_BODY()

	UE_API UTemplateSequencePlayer(const FObjectInitializer&);

public:

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player", meta = (WorldContext = "WorldContextObject", DynamicOutputParam = "OutActor"))
	static UE_API UTemplateSequencePlayer* CreateTemplateSequencePlayer(UObject* WorldContextObject, UTemplateSequence* TemplateSequence, FMovieSceneSequencePlaybackSettings Settings, ATemplateSequenceActor*& OutActor);

public:

	UE_API void Initialize(UMovieSceneSequence* InSequence, UWorld* InWorld, const FMovieSceneSequencePlaybackSettings& InSettings);

	// IMovieScenePlayer interface
	UE_API virtual UObject* GetPlaybackContext() const override;

private:

	/** The world this player will spawn actors in, if needed. */
	TWeakObjectPtr<UWorld> World;
};

/**
 * A spawn register that accepts a "wildcard" object.
 */
class FSequenceCameraShakeSpawnRegister : public FMovieSceneSpawnRegister
{
public:
	void SetSpawnedObject(UObject* InObject) { SpawnedObject = InObject; }

	virtual UObject* SpawnObject(FMovieSceneSpawnable&, FMovieSceneSequenceIDRef, TSharedRef<const UE::MovieScene::FSharedPlaybackState>) override { return SpawnedObject.Get(); }
	virtual void DestroySpawnedObject(UObject&, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding) override {}

#if WITH_EDITOR
	virtual bool CanSpawnObject(UClass* InClass) const override { return SpawnedObject.IsValid() && SpawnedObject.Get()->GetClass()->IsChildOf(InClass); }
#endif

private:
	FWeakObjectPtr SpawnedObject;
};

#undef UE_API
