// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequenceSpawnRegister.h"

#define UE_API TEMPLATESEQUENCE_API

/** Movie scene spawn register that knows how to handle spawning objects (actors) for a template sequence  */
class FTemplateSequenceSpawnRegister : public FLevelSequenceSpawnRegister
{
public:

	UE_API FTemplateSequenceSpawnRegister();
};

#undef UE_API
