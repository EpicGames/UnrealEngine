// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tracks/MovieSceneSubTrack.h"
#include "TemplateSequenceTrack.generated.h"

class UTemplateSequence;
class UTemplateSequenceSection;
struct FMovieSceneEvaluationTrack;

UCLASS(MinimalAPI)
class UTemplateSequenceTrack
	: public UMovieSceneSubTrack
{
public:
	GENERATED_BODY()

	UTemplateSequenceTrack(const FObjectInitializer& ObjectInitializer);

	virtual UMovieSceneSection* AddNewTemplateSequenceSection(FFrameNumber KeyTime, UTemplateSequence* InSequence);

	// UMovieSceneTrack interface
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif

private:
	static uint16 GetEvaluationPriority() { return uint16(0x1000); }  // One more than spawn tracks.
};
