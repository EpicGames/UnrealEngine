// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_CameraAnimationSequence.h"
#include "TemplateSequenceEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CameraAnimationSequence)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void UAssetDefinition_CameraAnimationSequence::InitializeToolkitParams(FTemplateSequenceToolkitParams& ToolkitParams) const
{
    UAssetDefinition_TemplateSequence::InitializeToolkitParams(ToolkitParams);

    ToolkitParams.bCanChangeBinding = false;
}

#undef LOCTEXT_NAMESPACE
