// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraAnimationSequence.h"
#include "AssetDefinitionDefault.h"
#include "AssetDefinition_TemplateSequence.h"

#include "AssetDefinition_CameraAnimationSequence.generated.h"

struct FTemplateSequenceToolkitParams;

UCLASS()
class UAssetDefinition_CameraAnimationSequence : public UAssetDefinition_TemplateSequence
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CameraAnimationSequence", "Camera Animation Sequence"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCameraAnimationSequence::StaticClass(); }
	// UAssetDefinition End

protected:

	virtual void InitializeToolkitParams(FTemplateSequenceToolkitParams& ToolkitParams) const override;
};
