// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_TemplateSequence.h"
#include "TemplateSequenceEditorToolkit.h"
#include "Styles/TemplateSequenceEditorStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_TemplateSequence)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FAssetOpenSupport UAssetDefinition_TemplateSequence::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit, EToolkitMode::WorldCentric); 
}

EAssetCommandResult UAssetDefinition_TemplateSequence::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	UWorld* WorldContext = nullptr;
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::Editor)
		{
			WorldContext = Context.World();
			break;
		}
	}

	if (!ensure(WorldContext))
	{
		return EAssetCommandResult::Handled;
	}

	for (UTemplateSequence* TemplateSequence : OpenArgs.LoadObjects<UTemplateSequence>())
	{
		TSharedRef<FTemplateSequenceEditorToolkit> Toolkit = MakeShareable(new FTemplateSequenceEditorToolkit(FTemplateSequenceEditorStyle::Get()));
		FTemplateSequenceToolkitParams ToolkitParams;
		InitializeToolkitParams(ToolkitParams);
		Toolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, TemplateSequence, ToolkitParams);
	}	

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
