// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TemplateSequence.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_TemplateSequence.generated.h"

struct FTemplateSequenceToolkitParams;

UCLASS()
class UAssetDefinition_TemplateSequence : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TemplateSequence", "Template Sequence"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(200, 80, 80)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTemplateSequence::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Cinematics };
		return Categories;
	}
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End

protected:

	virtual void InitializeToolkitParams(FTemplateSequenceToolkitParams& ToolkitParams) const {}
};
