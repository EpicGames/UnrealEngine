// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraAnimationSequenceCustomization.h"
#include "CineCameraActor.h"
#include "Customizations/TemplateSequenceCustomizationBase.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/TemplateSequenceEditorSettings.h"
#include "Styles/TemplateSequenceEditorStyle.h"
#include "Systems/TemplateSequenceCameraPreviewSystem.h"

#define LOCTEXT_NAMESPACE "CameraAnimationSequenceCustomization"

void FCameraAnimationSequenceCustomization::RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder)
{
	FTemplateSequenceCustomizationBase::RegisterSequencerCustomization(Builder);

	CameraActorClasses = { ACameraActor::StaticClass(), ACineCameraActor::StaticClass() };

	FSequencerCustomizationInfo Customization;

	TSharedRef<FExtender> ToolbarExtender = MakeShared<FExtender>();
	ToolbarExtender->AddToolBarExtension("BaseCommands", EExtensionHook::After, nullptr, FToolBarExtensionDelegate::CreateRaw(this, &FCameraAnimationSequenceCustomization::ExtendToolbar));
	Customization.ToolbarExtender = ToolbarExtender;

	TSharedRef<FExtender> ViewMenuExtender = MakeShared<FExtender>();
	ViewMenuExtender->AddMenuExtension("SequencerSettings", EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateRaw(this, &FCameraAnimationSequenceCustomization::ExtendViewMenu));
	Customization.ViewMenuExtender = ViewMenuExtender;

	Customization.OnAssetsDrop.BindLambda([](const TArray<UObject*>&, const FAssetDragDropOp&) -> ESequencerDropResult { return ESequencerDropResult::DropDenied; });
	Customization.OnClassesDrop.BindLambda([](const TArray<TWeakObjectPtr<UClass>>&, const FClassDragDropOp&) -> ESequencerDropResult { return ESequencerDropResult::DropDenied; });
	Customization.OnActorsDrop.BindLambda([](const TArray<TWeakObjectPtr<AActor>>&, const FActorDragDropOp&) -> ESequencerDropResult { return ESequencerDropResult::DropDenied; });

	Builder.AddCustomization(Customization);
}

void FCameraAnimationSequenceCustomization::UnregisterSequencerCustomization()
{
	FTemplateSequenceCustomizationBase::UnregisterSequencerCustomization();
}

void FCameraAnimationSequenceCustomization::ExtendViewMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("CameraAnimations", LOCTEXT("CameraAnimations", "Camera Animations"));
	{
		MenuBuilder.AddMenuEntry(
				LOCTEXT("CameraAdditiveToViewport", "Base Cameras on Viewport"),
				LOCTEXT("CameraAdditiveToViewportToolTip", "Play additive camera sections on top of viewport position"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FCameraAnimationSequenceCustomization::ToggleCameraInitiallyAdditiveToViewport),
					FCanExecuteAction(),
					FIsActionChecked::CreateRaw(this, &FCameraAnimationSequenceCustomization::IsCameraInitiallyAdditiveToViewport)),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();
}

void FCameraAnimationSequenceCustomization::ExtendToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginStyleOverride("SequencerToolBar");
	ToolbarBuilder.BeginSection("CameraAnimations", true);
	{
		ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateRaw(this, &FCameraAnimationSequenceCustomization::GetBoundCameraClassMenuContent),
				LOCTEXT("CameraTypePicker", "Camera Type"),
				LOCTEXT("CameraTypePickerTooltip", "Change the base camera actor type that this template sequence can bind to"),
				FSlateIcon(FTemplateSequenceEditorStyle::Get()->GetStyleSetName(), "TemplateSequenceEditor.Chain"));

		ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateRaw(this, &FCameraAnimationSequenceCustomization::SetCameraAdditiveToViewportOffset),
					FCanExecuteAction::CreateRaw(this, &FCameraAnimationSequenceCustomization::CanSetCameraAdditiveToViewportOffset)),
				NAME_None,
				FText::GetEmpty(),
				LOCTEXT("SetCameraAdditiveToViewportOffsetToolTip", "Base Cameras to Current Viewport Transform"),
				FSlateIcon(FTemplateSequenceEditorStyle::Get()->GetStyleSetName(), "SetViewportPreviewOffset"));
	}
	ToolbarBuilder.EndSection();
	ToolbarBuilder.EndStyleOverride();
}

TSharedRef<SWidget> FCameraAnimationSequenceCustomization::GetBoundCameraClassMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (UClass* CameraActorClass : CameraActorClasses)
	{
		MenuBuilder.AddMenuEntry(
				FText::FromString(CameraActorClass->GetName()),
				LOCTEXT("CameraTypeChoice", "Choose this type of camera as the type this template sequence can bind to"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, CameraActorClass]() { 
						OnBoundActorClassPicked(CameraActorClass); 
						}),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([this, CameraActorClass]() {
						return IsBoundToActorClass(CameraActorClass) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
						})
					),
				NAME_None,
				EUserInterfaceActionType::RadioButton
				);
	}

	return MenuBuilder.MakeWidget();
}

bool FCameraAnimationSequenceCustomization::IsBoundToActorClass(UClass* InClass)
{
	UClass* BoundActorClass = GetBoundActorClass();
	return BoundActorClass == InClass;
}

void FCameraAnimationSequenceCustomization::ToggleCameraInitiallyAdditiveToViewport()
{
	UTemplateSequenceEditorSettings* Settings = GetMutableDefault<UTemplateSequenceEditorSettings>();
	Settings->bCameraInitiallyAdditiveToViewport = !Settings->bCameraInitiallyAdditiveToViewport;
	Settings->SaveConfig();

	UTemplateSequenceCameraPreviewSystem::DisableNextFrame();
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencer())
	{
		SequencerPtr->ForceEvaluate();
	}
}

bool FCameraAnimationSequenceCustomization::IsCameraInitiallyAdditiveToViewport()
{
	UTemplateSequenceEditorSettings* Settings = GetMutableDefault<UTemplateSequenceEditorSettings>();
	return Settings->bCameraInitiallyAdditiveToViewport;
}

void FCameraAnimationSequenceCustomization::SetCameraAdditiveToViewportOffset()
{
	UTemplateSequenceCameraPreviewSystem::EnableNextFrame();
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencer())
	{
		SequencerPtr->ForceEvaluate();
	}
}

bool FCameraAnimationSequenceCustomization::CanSetCameraAdditiveToViewportOffset()
{
	return IsCameraInitiallyAdditiveToViewport();
}

#undef LOCTEXT_NAMESPACE

