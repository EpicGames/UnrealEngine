// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TemplateSequenceCustomizationBase.h"

class FSequencerCustomizationBuilder;
class SWidget;

class FToolBarBuilder;

/**
 * The sequencer customization for camera animation sequences.
 */
class FCameraAnimationSequenceCustomization : public FTemplateSequenceCustomizationBase
{
private:
	virtual void RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder) override;
	virtual void UnregisterSequencerCustomization() override;

	void ExtendViewMenu(FMenuBuilder& MenuBuilder);
	void ExtendToolbar(FToolBarBuilder& ToolbarBuilder);

	TSharedRef<SWidget> GetBoundCameraClassMenuContent();
	bool IsBoundToActorClass(UClass* InClass);

	void ToggleCameraInitiallyAdditiveToViewport();
	bool IsCameraInitiallyAdditiveToViewport();

	void SetCameraAdditiveToViewportOffset();
	bool CanSetCameraAdditiveToViewportOffset();

	TArray<UClass*> CameraActorClasses;
};

