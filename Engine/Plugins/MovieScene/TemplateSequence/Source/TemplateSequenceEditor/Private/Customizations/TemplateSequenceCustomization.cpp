// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateSequenceCustomization.h"
#include "ClassViewerModule.h"
#include "Customizations/TemplateSequenceCustomizationBase.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/ClassDragDropOp.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "Styles/TemplateSequenceEditorStyle.h"

#define LOCTEXT_NAMESPACE "TemplateSequenceCustomization"

void FTemplateSequenceCustomization::RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder) 
{
	FTemplateSequenceCustomizationBase::RegisterSequencerCustomization(Builder);

	FSequencerCustomizationInfo Customization;

	TSharedRef<FExtender> ToolbarExtender = MakeShared<FExtender>();
	ToolbarExtender->AddToolBarExtension("BaseCommands", EExtensionHook::After, nullptr, FToolBarExtensionDelegate::CreateRaw(this, &FTemplateSequenceCustomization::ExtendSequencerToolbar));
	Customization.ToolbarExtender = ToolbarExtender;

	Customization.OnAssetsDrop.BindRaw(this, &FTemplateSequenceCustomization::OnSequencerAssetsDrop);
	Customization.OnClassesDrop.BindRaw(this, &FTemplateSequenceCustomization::OnSequencerClassesDrop);
	Customization.OnActorsDrop.BindRaw(this, &FTemplateSequenceCustomization::OnSequencerActorsDrop);

	Builder.AddCustomization(Customization);
}

void FTemplateSequenceCustomization::UnregisterSequencerCustomization()
{
	FTemplateSequenceCustomizationBase::UnregisterSequencerCustomization();
}

void FTemplateSequenceCustomization::ExtendSequencerToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginStyleOverride("SequencerToolBar");
	ToolbarBuilder.BeginSection("TemplateSequence", true);

	ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateRaw(this, &FTemplateSequenceCustomization::GetBoundActorClassMenuContent),
			LOCTEXT("BoundActorClassPicker", "Bound Actor Class"),
			LOCTEXT("BoundActorClassPickerTooltip", "Change the base actor type that this template sequence can bind to"),
			FSlateIcon(FTemplateSequenceEditorStyle::Get()->GetStyleSetName(), "TemplateSequenceEditor.Chain"));

	ToolbarBuilder.EndSection();
	ToolbarBuilder.EndStyleOverride();
}

TSharedRef<SWidget> FTemplateSequenceCustomization::GetBoundActorClassMenuContent()
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	Options.bShowObjectRootClass = true;
	Options.bIsActorsOnly = true;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::Dynamic;

	TSharedRef<SWidget> ClassPicker = ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateRaw(this, &FTemplateSequenceCustomization::OnBoundActorClassPicked));

	return SNew(SBox)
		.WidthOverride(350.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.MaxHeight(400.0f)
			.AutoHeight()
			[
				ClassPicker
			]
		];
}

bool FTemplateSequenceCustomization::OnSequencerReceivedDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, FReply& OutReply)
{
	bool bIsDragSupported = false;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid() && (
		(Operation->IsOfType<FAssetDragDropOp>() && StaticCastSharedPtr<FAssetDragDropOp>(Operation)->GetAssetPaths().Num() <= 1) ||
		(Operation->IsOfType<FClassDragDropOp>() && StaticCastSharedPtr<FClassDragDropOp>(Operation)->ClassesToDrop.Num() <= 1) ||
		(Operation->IsOfType<FActorDragDropOp>() && StaticCastSharedPtr<FActorDragDropOp>(Operation)->Actors.Num() <= 1)))
	{
		bIsDragSupported = true;
	}

	OutReply = (bIsDragSupported ? FReply::Handled() : FReply::Unhandled());
	return true;
}

ESequencerDropResult FTemplateSequenceCustomization::OnSequencerAssetsDrop(const TArray<UObject*>& Assets, const FAssetDragDropOp& DragDropOp)
{
	if (Assets.Num() > 0)
	{
		// Only drop the first asset.
		UObject* CurObject = Assets[0];

		// TODO: check for dropping a sequence?

		ChangeActorBinding(CurObject, DragDropOp.GetActorFactory());

		return ESequencerDropResult::DropHandled;
	}

	return ESequencerDropResult::Unhandled;
}

ESequencerDropResult FTemplateSequenceCustomization::OnSequencerClassesDrop(const TArray<TWeakObjectPtr<UClass>>& Classes, const FClassDragDropOp& DragDropOp)
{
	if (Classes.Num() > 0 && Classes[0].IsValid())
	{
		// Only drop the first class.
		UClass* CurClass = Classes[0].Get();

		ChangeActorBinding(CurClass);

		return ESequencerDropResult::DropHandled;
	}
	return ESequencerDropResult::Unhandled;
}

ESequencerDropResult FTemplateSequenceCustomization::OnSequencerActorsDrop(const TArray<TWeakObjectPtr<AActor>>& Actors, const FActorDragDropOp& DragDropOp)
{
	return ESequencerDropResult::Unhandled;
}

#undef LOCTEXT_NAMESPACE
