// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TemplateSequenceCustomizationBase.h"

class FSequencerCustomizationBuilder;
class SWidget;
enum class ESequencerDropResult;

class FToolBarBuilder;

/**
 * The sequencer customization for template sequences. 
 */
class FTemplateSequenceCustomization : public FTemplateSequenceCustomizationBase
{
private:
	virtual void RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder) override;
	virtual void UnregisterSequencerCustomization() override;

	void ExtendSequencerToolbar(FToolBarBuilder& ToolbarBuilder);

	bool OnSequencerReceivedDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, FReply& OutReply);
	ESequencerDropResult OnSequencerAssetsDrop(const TArray<UObject*>& Assets, const FAssetDragDropOp& DragDropOp);
	ESequencerDropResult OnSequencerClassesDrop(const TArray<TWeakObjectPtr<UClass>>& Classes, const FClassDragDropOp& DragDropOp);
	ESequencerDropResult OnSequencerActorsDrop(const TArray<TWeakObjectPtr<AActor>>& Actors, const FActorDragDropOp& DragDropOp);
	TSharedRef<SWidget> GetBoundActorClassMenuContent();
};

