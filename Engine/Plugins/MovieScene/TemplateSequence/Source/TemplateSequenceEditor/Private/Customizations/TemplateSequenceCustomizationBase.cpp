// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateSequenceCustomizationBase.h"

#include "BakingAnimationKeySettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "GameFramework/Actor.h"
#include "ISequencer.h"
#include "LevelSequenceFBXInterop.h"
#include "Misc/TemplateSequenceEditorUtil.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "ScopedTransaction.h"
#include "SequencerUtilities.h"
#include "TemplateSequence.h"
#include "Widgets/SLevelSequenceBakeTransform.h"

#define LOCTEXT_NAMESPACE "TemplateSequenceCustomizationBase"

void FTemplateSequenceCustomizationBase::RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder)
{
	WeakSequencer = Builder.GetSequencer().AsShared();
	TemplateSequence = Cast<UTemplateSequence>(&Builder.GetFocusedSequence());

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	Sequencer->OnCloseEvent().AddRaw(this, &FTemplateSequenceCustomizationBase::OnSequencerClosed);
	Sequencer->OnMovieSceneDataChanged().AddRaw(this, &FTemplateSequenceCustomizationBase::OnMovieSceneDataChanged);

	FSequencerCustomizationInfo BaseInfo;
	BaseInfo.OnPaste.BindRaw(this, &FTemplateSequenceCustomizationBase::OnPaste);
	BaseInfo.OnBuildObjectBindingContextMenu = FOnGetSequencerMenuExtender::CreateRaw(this, &FTemplateSequenceCustomizationBase::CreateObjectBindingContextMenuExtender);
	Builder.AddCustomization(BaseInfo);
}

void FTemplateSequenceCustomizationBase::UnregisterSequencerCustomization()
{
	if (WeakSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

		Sequencer->OnCloseEvent().RemoveAll(this);
		Sequencer->OnMovieSceneDataChanged().RemoveAll(this);
	}

	WeakSequencer = nullptr;
	TemplateSequence = nullptr;
}

UClass* FTemplateSequenceCustomizationBase::GetBoundActorClass() const
{
	return TemplateSequence ? TemplateSequence->BoundActorClass.Get() : nullptr;
}

FText FTemplateSequenceCustomizationBase::GetBoundActorClassName() const
{
	const UClass* BoundActorClass = GetBoundActorClass();
	return BoundActorClass ? BoundActorClass->GetDisplayNameText() : FText::FromName(NAME_None);
}

void FTemplateSequenceCustomizationBase::OnBoundActorClassPicked(UClass* ChosenClass)
{
	FSlateApplication::Get().DismissAllMenus();

	if (TemplateSequence != nullptr)
	{
		ChangeActorBinding(ChosenClass);

		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}

void FTemplateSequenceCustomizationBase::ChangeActorBinding(UObject* Object, UActorFactory* ActorFactory, bool bSetupDefaults)
{
	const FScopedTransaction Transaction(LOCTEXT("ChangeActorBinding", "Change Template Sequence Actor Binding"));

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	FTemplateSequenceEditorUtil Util(TemplateSequence, *Sequencer);
	Util.ChangeActorBinding(Object, ActorFactory, bSetupDefaults);
}

ESequencerPasteSupport FTemplateSequenceCustomizationBase::OnPaste()
{
	// We don't support pasting folders or new object bindings.
	return ESequencerPasteSupport::Tracks | ESequencerPasteSupport::Sections;
}

TSharedPtr<FExtender> FTemplateSequenceCustomizationBase::CreateObjectBindingContextMenuExtender(UE::Sequencer::FViewModelPtr InViewModel)
{
	using namespace UE::Sequencer;

	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	TSharedPtr<FObjectBindingModel> ObjectBindingModel = InViewModel->CastThisShared<FObjectBindingModel>();
	Extender->AddMenuExtension(
			"ObjectBindingActions", EExtensionHook::Before, nullptr,
			FMenuExtensionDelegate::CreateRaw(this, &FTemplateSequenceCustomizationBase::ExtendObjectBindingContextMenu, ObjectBindingModel));
	return Extender.ToSharedPtr();
}

void FTemplateSequenceCustomizationBase::ExtendObjectBindingContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<UE::Sequencer::FObjectBindingModel> ObjectBindingModel)
{
	using namespace UE::Sequencer;

	MenuBuilder.BeginSection("Import/Export", LOCTEXT("ImportExportMenuSectionName", "Import/Export"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("BakeTransform", "Bake Transform"),
		LOCTEXT("BakeTransformTooltip", "Bake transform in world space, removing any existing transform and attach tracks"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this] {
				const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer.IsValid())
				{
					return;
				}

				UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
				UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
				if (!MovieScene)
				{
					return;
				}

				TArray<FGuid> ObjectBindings;
				Sequencer->GetSelectedObjects(ObjectBindings);

				TArray<FMovieSceneBindingProxy> BindingProxies;
				for (FGuid Guid : ObjectBindings)
				{
					BindingProxies.Add(FMovieSceneBindingProxy(Guid, Sequence));
				}

				static FBakingAnimationKeySettings Settings; //reuse the settings except for the range
				Settings.StartFrame = UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange());
				Settings.EndFrame = UE::MovieScene::DiscreteExclusiveUpper(MovieScene->GetPlaybackRange());

				TSharedRef<SLevelSequenceBakeTransform> BakeWidget =
					SNew(SLevelSequenceBakeTransform)
					.Settings(Settings)
					.Sequencer(Sequencer.Get())
					.OnBake_Lambda([this, &BindingProxies, Sequencer](FBakingAnimationKeySettings InSettings)
						{
							if (FSequencerUtilities::BakeTransform(Sequencer.ToSharedRef(), BindingProxies, InSettings))
							{
								Settings = InSettings;
							}

							return FReply::Handled();
						});

				BakeWidget->OpenDialog(true);
			})
		));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ImportFBX", "Import..."),
		LOCTEXT("ImportFBXTooltip", "Import FBX animation to this object"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this] {
				const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer.IsValid())
				{
					return;
				}

				FLevelSequenceFBXInterop Interop(Sequencer.ToSharedRef());
					Interop.ImportFBXOntoSelectedNodes();
				})
		));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ExportFBX", "Export..."),
		LOCTEXT("ExportFBXTooltip", "Export FBX animation from this object"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this] {
				const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer.IsValid())
				{
					return;
				}
				
				FLevelSequenceFBXInterop Interop(Sequencer.ToSharedRef());
					Interop.ExportFBX();
				})
		));

	MenuBuilder.EndSection();
}

void FTemplateSequenceCustomizationBase::OnMovieSceneDataChanged(EMovieSceneDataChangeType ChangeType)
{
	if (ChangeType == EMovieSceneDataChangeType::TrackValueChanged || ChangeType == EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately)
	{
		return;
	}

	// Ensure the BoundActorClass is up to date with whatever change might have occured.
	// This is mostly only needed when the user deletes or cuts the root object binding and we are suddenly left
	// with an empty template sequence (in which case we need to set BoundActorClass to null).
	const UObject* RootSpawnableTemplate = TemplateSequence->GetRootObjectSpawnableTemplate();	
	const UClass* RootObjectClass = Cast<const UClass>(RootSpawnableTemplate);
	if (RootSpawnableTemplate && !RootObjectClass)
	{
		RootObjectClass = RootSpawnableTemplate->GetClass();
	}

	if (TemplateSequence->BoundActorClass != RootObjectClass)
	{
		FProperty* BoundActorClassProperty = FindFProperty<FProperty>(UTemplateSequence::StaticClass(), GET_MEMBER_NAME_CHECKED(UTemplateSequence, BoundActorClass));
		TemplateSequence->PreEditChange(BoundActorClassProperty);

		TemplateSequence->BoundActorClass = RootObjectClass;

		FPropertyChangedEvent PropertyEvent(BoundActorClassProperty, EPropertyChangeType::ValueSet);
		TemplateSequence->PostEditChangeProperty(PropertyEvent);
	}
}

void FTemplateSequenceCustomizationBase::OnSequencerClosed(TSharedRef<ISequencer> InSequencer)
{
	if (WeakSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

		if (InSequencer == Sequencer)
		{
			Sequencer->OnCloseEvent().RemoveAll(this);
			Sequencer->OnMovieSceneDataChanged().RemoveAll(this);
		}

		WeakSequencer = nullptr;
	}
}

#undef LOCTEXT_NAMESPACE

