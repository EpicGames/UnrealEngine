// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "SequencerCustomizationManager.h"

class UActorFactory;
enum class EMovieSceneDataChangeType;

class UTemplateSequence;

/**
 * Base implementation of a template sequence customization for Sequencer.
 */
class FTemplateSequenceCustomizationBase : public ISequencerCustomization
{
protected:
	virtual void RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder) override;
	virtual void UnregisterSequencerCustomization() override;

	UClass* GetBoundActorClass() const;
	FText GetBoundActorClassName() const;
	void OnBoundActorClassPicked(UClass* ChosenClass);
	void ChangeActorBinding(UObject* Object, UActorFactory* ActorFactory = nullptr, bool bSetupDefaults = true);

	TSharedPtr<ISequencer> GetSequencer() const { return WeakSequencer.Pin(); }
	UTemplateSequence* GetTemplateSequence() const { return TemplateSequence; }

	TSharedPtr<FExtender> CreateObjectBindingContextMenuExtender(UE::Sequencer::FViewModelPtr InViewModel);
	void ExtendObjectBindingContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<UE::Sequencer::FObjectBindingModel> ObjectBindingModel);

private:
	ESequencerPasteSupport OnPaste();
	void OnMovieSceneDataChanged(EMovieSceneDataChangeType ChangeType);
	void OnSequencerClosed(TSharedRef<ISequencer> InSequencer);

	TWeakPtr<ISequencer> WeakSequencer;

	UTemplateSequence* TemplateSequence;
};

