// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CameraAnimationSequenceFactoryNew.h"
#include "CameraAnimationSequence.h"
#include "CineCameraActor.h"
#include "Factories/TemplateSequenceFactoryUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraAnimationSequenceFactoryNew)

UCameraAnimationSequenceFactoryNew::UCameraAnimationSequenceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCameraAnimationSequence::StaticClass();
}

UObject* UCameraAnimationSequenceFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FTemplateSequenceFactoryUtil::CreateTemplateSequence(InParent, Name, Flags, UCameraAnimationSequence::StaticClass(), ACineCameraActor::StaticClass());
}

bool UCameraAnimationSequenceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

