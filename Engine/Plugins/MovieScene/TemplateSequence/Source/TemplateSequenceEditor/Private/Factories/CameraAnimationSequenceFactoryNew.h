// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "CameraAnimationSequenceFactoryNew.generated.h"

/**
 * Implements a factory for UCameraAnimationSequence objects.
 */
UCLASS(hidecategories = Object)
class UCameraAnimationSequenceFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
};

