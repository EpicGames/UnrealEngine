// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/TemplateSequenceFactoryNew.h"
#include "Factories/TemplateSequenceFactoryUtil.h"
#include "GameFramework/Actor.h"
#include "Kismet2/SClassPickerDialog.h"
#include "TemplateSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateSequenceFactoryNew)

#define LOCTEXT_NAMESPACE "TemplateSequenceFactory"

UTemplateSequenceFactoryNew::UTemplateSequenceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UTemplateSequence::StaticClass();
}

UObject* UTemplateSequenceFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FTemplateSequenceFactoryUtil::CreateTemplateSequence(InParent, Name, Flags, UTemplateSequence::StaticClass(), BoundActorClass);
}

bool UTemplateSequenceFactoryNew::ConfigureProperties()
{
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	Options.bShowObjectRootClass = true;
	Options.bIsActorsOnly = true;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::Dynamic;

	const FText TitleText = LOCTEXT("CreateTemplateSequenceOptions", "Pick Root Object Binding Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, AActor::StaticClass());
	if (bPressedOk)
	{
		BoundActorClass = ChosenClass;
	}

	return bPressedOk;
}

bool UTemplateSequenceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE

