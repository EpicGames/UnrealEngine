// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "TemplateSequenceFactoryNew.generated.h"

class UMovieSceneSequence;

/**
 * Implements a factory for UTemplateSequence objects.
 */
UCLASS(hidecategories = Object)
class UTemplateSequenceFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;

	// The root object binding class of the created template sequence.
	UPROPERTY(EditAnywhere, Category=TemplateSequenceFactory)
	TSubclassOf<UObject> BoundActorClass;
};
