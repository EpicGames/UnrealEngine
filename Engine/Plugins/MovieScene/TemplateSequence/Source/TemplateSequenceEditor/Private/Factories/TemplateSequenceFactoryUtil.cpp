// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/TemplateSequenceFactoryUtil.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ILevelSequenceModule.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneToolsProjectSettings.h"
#include "Sections/MovieSceneBoolSection.h"
#include "TemplateSequence.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "TemplateSequenceFactoryUtil"

UTemplateSequence* FTemplateSequenceFactoryUtil::CreateTemplateSequence(UObject* InParent, FName InName, EObjectFlags InFlags, TSubclassOf<UTemplateSequence> InSequenceClass, TSubclassOf<UObject> InObjectTemplateClass)
{
	UTemplateSequence* NewTemplateSequence = NewObject<UTemplateSequence>(InParent, InSequenceClass, InName, InFlags | RF_Transactional);
	NewTemplateSequence->Initialize();

	// Set up some sensible defaults
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	FFrameRate TickResolution = NewTemplateSequence->GetMovieScene()->GetTickResolution();
	NewTemplateSequence->GetMovieScene()->SetPlaybackRange((ProjectSettings->DefaultStartTime*TickResolution).FloorToFrame(), (ProjectSettings->DefaultDuration*TickResolution).FloorToFrame().Value);

	if (InObjectTemplateClass)
	{
		NewTemplateSequence->BoundActorClass = InObjectTemplateClass;

		InitializeSpawnable(NewTemplateSequence, InObjectTemplateClass);
	}

	return NewTemplateSequence;
}

void FTemplateSequenceFactoryUtil::InitializeSpawnable(UMovieSceneSequence* InTemplateSequence, TSubclassOf<UObject> InObjectTemplateClass)
{
	// Grab the object spawners, so we can create a new actor of the desired class.
	TArray<TSharedRef<IMovieSceneObjectSpawner>> ObjectSpawners;
	ILevelSequenceModule& LevelSequenceModule = FModuleManager::LoadModuleChecked<ILevelSequenceModule>("LevelSequence");
	LevelSequenceModule.GenerateObjectSpawners(ObjectSpawners);

	// Run the object spawners until we find one that works.
	UMovieScene* MovieScene = InTemplateSequence->GetMovieScene();
	check(MovieScene);

	TValueOrError<FNewSpawnable, FText> Result = MakeError(LOCTEXT("NoSpawnerFound", "No spawner found to create new spawnable type"));

	for (TSharedRef<IMovieSceneObjectSpawner> Spawner : ObjectSpawners)
	{
		TValueOrError<FNewSpawnable, FText> CurResult = Spawner->CreateNewSpawnableType(*InObjectTemplateClass.Get(), *MovieScene, nullptr);
		if (CurResult.IsValid())
		{
			Result = CurResult;
			break;
		}
	}
	
	// Bail out if we haven't found any good spawner.
	if (!Result.IsValid())
	{
		FNotificationInfo Info(Result.GetError());
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);

		return;
	}

	// Create the spawnable.
	FNewSpawnable& NewSpawnable = Result.GetValue();

	NewSpawnable.Name = MovieSceneHelpers::MakeUniqueSpawnableName(MovieScene, NewSpawnable.Name);			

	const FGuid NewGuid = MovieScene->AddSpawnable(NewSpawnable.Name, *NewSpawnable.ObjectTemplate);
	
	// Create a new spawn section.
	if (ensure(NewGuid.IsValid()))
	{
		UMovieSceneSpawnTrack* NewSpawnTrack = Cast<UMovieSceneSpawnTrack>(MovieScene->AddTrack(UMovieSceneSpawnTrack::StaticClass(), NewGuid));
		UMovieSceneBoolSection* NewSpawnSection = Cast<UMovieSceneBoolSection>(NewSpawnTrack->CreateNewSection());
		NewSpawnSection->GetChannel().SetDefault(true);
		NewSpawnSection->SetRange(TRange<FFrameNumber>::All());
		NewSpawnTrack->AddSection(*NewSpawnSection);
		NewSpawnTrack->SetObjectId(NewGuid);
	}
}

#undef LOCTEXT_NAMESPACE
