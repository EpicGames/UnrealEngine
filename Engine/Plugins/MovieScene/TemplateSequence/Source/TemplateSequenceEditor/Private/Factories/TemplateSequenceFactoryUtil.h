// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

template <typename T> class TSubclassOf;

class UMovieSceneSequence;
class UObject;
class UTemplateSequence;

/**
 * Utility class for template sequence factories
 */
class FTemplateSequenceFactoryUtil
{
public:

	// Create a new template sequence
	static UTemplateSequence* CreateTemplateSequence(UObject* InParent, FName InName, EObjectFlags Flags, TSubclassOf<UTemplateSequence> InSequenceClass, TSubclassOf<UObject> InObjectTemplateClass);

private:

	// Create a new spawnable for the chosen root object class.
	static void InitializeSpawnable(UMovieSceneSequence* InTemplateSequence, TSubclassOf<UObject> InObjectTemplateClass);
};

