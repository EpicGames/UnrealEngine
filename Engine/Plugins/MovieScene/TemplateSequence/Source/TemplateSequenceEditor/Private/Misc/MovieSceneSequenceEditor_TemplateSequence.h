// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequenceEditor.h"

struct FMovieSceneSequenceEditor_TemplateSequence : FMovieSceneSequenceEditor
{
};
