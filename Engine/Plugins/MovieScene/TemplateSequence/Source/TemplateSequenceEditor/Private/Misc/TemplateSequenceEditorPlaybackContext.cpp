// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TemplateSequenceEditorPlaybackContext.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "TemplateSequenceEditorPlaybackContext"

UObject* FTemplateSequenceEditorPlaybackContext::GetPlaybackContext() const
{
	UWorld* Context = WeakCurrentContext.Get();
	if (Context)
	{
		return Context;
	}

	Context = ComputePlaybackContext();
	check(Context);
	WeakCurrentContext = Context;
	return Context;
}

UWorld* FTemplateSequenceEditorPlaybackContext::ComputePlaybackContext()
{
	const bool bIsSimulatingInEditor = GEditor && GEditor->bIsSimulatingInEditor;

	UWorld* EditorWorld = nullptr;

	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			UWorld* ThisWorld = Context.World();
			if (ThisWorld)
			{
				EditorWorld = ThisWorld;
				break;
			}
		}
		else if (Context.WorldType == EWorldType::Editor)
		{
			EditorWorld = Context.World();
		}
	}

	check(EditorWorld);
	return EditorWorld;
}

#undef LOCTEXT_NAMESPACE
