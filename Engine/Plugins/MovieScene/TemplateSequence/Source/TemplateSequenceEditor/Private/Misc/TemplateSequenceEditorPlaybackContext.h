// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"

class UWorld;

/**
 * Class that manages the current UWorld context that a level-sequence editor should use for playback
 */
class FTemplateSequenceEditorPlaybackContext : public TSharedFromThis<FTemplateSequenceEditorPlaybackContext>
{
public:

	UObject* GetPlaybackContext() const;

private:

	static UWorld* ComputePlaybackContext();
	
	/** Mutable cached context pointer */
	mutable TWeakObjectPtr<UWorld> WeakCurrentContext;
};
