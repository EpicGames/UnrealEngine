// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TemplateSequenceEditorSettings.generated.h"

/**
 * Template Sequence Editor settings.
 */
UCLASS(config=EditorPerProjectUserSettings)
class UTemplateSequenceEditorSettings
	: public UObject
{
	GENERATED_BODY()

public:

	UTemplateSequenceEditorSettings(const FObjectInitializer& ObjectInitializer);

	/** Whether to show "outdated assets" by default in the camera animation track's asset picker. */
	UPROPERTY(config)
	bool bShowOutdatedAssetsInCameraAnimationTrackEditor = false;

	/** Whether to set the viewport position as the initial value for additive camera sections. */
	UPROPERTY(config)
	bool bCameraInitiallyAdditiveToViewport = true;
};

