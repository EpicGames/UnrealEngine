// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/LevelSequenceEditorSpawnRegister.h"

/** Movie scene spawn register that knows how to handle spawning objects (actors) for a template sequence  */
class FTemplateSequenceEditorSpawnRegister : public FLevelSequenceEditorSpawnRegister
{
#if WITH_EDITOR
	virtual bool CanConvertSpawnableToPossessable(FMovieSceneSpawnable& Spawnable) const override;
#endif
};
