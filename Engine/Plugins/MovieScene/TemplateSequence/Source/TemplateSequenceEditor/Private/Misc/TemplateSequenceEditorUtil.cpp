// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TemplateSequenceEditorUtil.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "MovieSceneSpawnable.h"
#include "TemplateSequence.h"

FTemplateSequenceEditorUtil::FTemplateSequenceEditorUtil(UTemplateSequence* InTemplateSequence, ISequencer& InSequencer)
	: TemplateSequence(InTemplateSequence)
	, Sequencer(InSequencer)
{
	check(InTemplateSequence != nullptr);
}

void FTemplateSequenceEditorUtil::ChangeActorBinding(UObject* Object, UActorFactory* ActorFactory, bool bSetupDefaults)
{
	if (!ensure(Object != nullptr))
	{
		return;
	}

	// Get the currently edited sequence. It isn't necessarily TemplateSequence, as we could have template 
	// sequences inside template sequences.
	UMovieSceneSequence* FocusedSequence = Sequencer.GetFocusedMovieSceneSequence();
	if (!ensure(FocusedSequence))
	{
		return;
	}

	UTemplateSequence* FocusedTemplateSequence = Cast<UTemplateSequence>(FocusedSequence);
	if (!FocusedTemplateSequence)
	{
		return;
	}

	// See if we have anything to do in the first place.
	if (UClass* ChosenClass = Cast<UClass>(Object))
	{
		if (ChosenClass == FocusedTemplateSequence->BoundActorClass)
		{
			return;
		}
	}

	UMovieScene* MovieScene = FocusedTemplateSequence->GetMovieScene();
	check(MovieScene != nullptr);

	// See if we previously had a main object binding.
	FGuid PreviousSpawnableGuid;
	if (MovieScene->GetSpawnableCount() > 0)
	{
		PreviousSpawnableGuid = MovieScene->GetSpawnable(0).GetGuid();
	}

	// Make the new spawnable object binding.
	FGuid NewSpawnableGuid = Sequencer.MakeNewSpawnable(*Object);
	FMovieSceneSpawnable* NewSpawnable = MovieScene->FindSpawnable(NewSpawnableGuid);
	if (!NewSpawnable)
	{
		return;
	}

	// Pre-notify that a change will occur.
	FProperty* BoundActorClassProperty = FindFProperty<FProperty>(UTemplateSequence::StaticClass(), GET_MEMBER_NAME_CHECKED(UTemplateSequence, BoundActorClass));
	FocusedTemplateSequence->PreEditChange(BoundActorClassProperty);

	// Update the BoundActorClass.
	if (Object->IsA<UClass>())
	{
		UClass* ChosenClass = StaticCast<UClass*>(Object);
		FocusedTemplateSequence->BoundActorClass = ChosenClass;
	}
	else
	{
		const UObject* SpawnableTemplate = NewSpawnable->GetObjectTemplate();
		if (ensure(SpawnableTemplate != nullptr))
		{
			FocusedTemplateSequence->BoundActorClass = SpawnableTemplate->GetClass();
		}
	}

	// If we had a previous one, move everything under it to the new binding, and clean up.
	if (PreviousSpawnableGuid.IsValid())
	{
		MovieScene->MoveBindingContents(PreviousSpawnableGuid, NewSpawnableGuid);

		if (MovieScene->RemoveSpawnable(PreviousSpawnableGuid))
		{
			FMovieSceneSpawnRegister& SpawnRegister = Sequencer.GetSpawnRegister();
			SpawnRegister.DestroySpawnedObject(PreviousSpawnableGuid, Sequencer.GetFocusedTemplateID(), Sequencer);
		}
	}

	// Notify that the change occured.
	FPropertyChangedEvent PropertyEvent(BoundActorClassProperty, EPropertyChangeType::ValueSet);
	FocusedTemplateSequence->PostEditChangeProperty(PropertyEvent);
}

