// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class UObject;

class ISequencer;
class UActorFactory;
class UTemplateSequence;

class FTemplateSequenceEditorUtil
{
public:
	FTemplateSequenceEditorUtil(UTemplateSequence* InTemplateSequence, ISequencer& InSequencer);

	void ChangeActorBinding(UObject* Object, UActorFactory* ActorFactory = nullptr, bool bSetupDefaults = true);

private:
	/** Template sequence to operate upon. */
	UTemplateSequence* TemplateSequence;

	/** The sequencer currently editing the sequence. */
	ISequencer& Sequencer;
};

