// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/TemplateSequenceEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

TSharedPtr<FTemplateSequenceEditorStyle> FTemplateSequenceEditorStyle::Singleton;

FTemplateSequenceEditorStyle::FTemplateSequenceEditorStyle()
	: FSlateStyleSet("TemplateSequenceEditorStyle")
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon48x48(48.0f, 48.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("MovieScene/TemplateSequence/Content"));

	// tab icons
	Set("TemplateSequenceEditor.Tabs.Sequencer", new IMAGE_BRUSH("icon_tab_sequencer_16x", Icon16x16));

	// asset thumbnail
	Set("ClassThumbnail.TemplateSequence", new IMAGE_BRUSH("TemplateSequence_16x", Icon16x16));
	Set("ClassThumbnail.TemplateSequence", new IMAGE_BRUSH("TemplateSequence_64x", Icon64x64));

	// toolbar icons
	Set("TemplateSequenceEditor.Chain", new IMAGE_BRUSH("Chain_24x", Icon24x24));
	Set("TemplateSequenceEditor.Chain.Small", new IMAGE_BRUSH("Chain_16x", Icon16x16));
	Set("SetViewportPreviewOffset", new IMAGE_BRUSH_SVG("SetViewportPreviewOffset", Icon24x24));
	Set("SetViewportPreviewOffset.Small", new IMAGE_BRUSH_SVG("SetViewportPreviewOffset", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FTemplateSequenceEditorStyle::~FTemplateSequenceEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

TSharedRef<FTemplateSequenceEditorStyle> FTemplateSequenceEditorStyle::Get()
{
	if (!Singleton.IsValid())
	{
		Singleton = MakeShareable(new FTemplateSequenceEditorStyle);
	}
	return Singleton.ToSharedRef();
}
