// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * Implements the visual style of the template sequence editor.
 */
class FTemplateSequenceEditorStyle final : public FSlateStyleSet
{
public:
	
	FTemplateSequenceEditorStyle();
	virtual ~FTemplateSequenceEditorStyle();

	static TSharedRef<FTemplateSequenceEditorStyle> Get();

private:

	static TSharedPtr<FTemplateSequenceEditorStyle> Singleton;
};

#undef IMAGE_BRUSH
