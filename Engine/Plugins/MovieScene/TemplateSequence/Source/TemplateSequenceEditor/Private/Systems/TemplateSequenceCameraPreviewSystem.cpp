// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/TemplateSequenceCameraPreviewSystem.h"

#include "Camera/CameraActor.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationExtension.h"
#include "EntitySystem/MovieSceneBlenderSystemTypes.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "LevelEditorViewport.h"
#include "Misc/TemplateSequenceEditorSettings.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSpawnable.h"
#include "MovieSceneTracksComponentTypes.h"
#include "TemplateSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateSequenceCameraPreviewSystem)

namespace UE::MovieScene
{

struct FSetInitialTransformValues
{
	TOptional<FTransform> TransformToSet;
	TArray<FMovieSceneEntityID> Entities;

	void ForEachAllocation(
			const FEntityAllocation* Allocation, 
			FReadEntityIDs EntityIDs,
			TWrite<FIntermediate3DTransform> InitialTransforms) const
	{
		if (!TransformToSet.IsSet())
		{
			return;
		}

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			FMovieSceneEntityID EntityID = EntityIDs[Index];
			if (Entities.Contains(EntityID))
			{
				FIntermediate3DTransform& InitialTransform = InitialTransforms[Index];
				InitialTransform = FIntermediate3DTransform(
						TransformToSet->GetLocation(),
						TransformToSet->GetRotation().Rotator(),
						TransformToSet->GetScale3D());
			}
		}
	}
};

struct FSetInitialTransformToViewport : FSetInitialTransformValues
{
	FSetInitialTransformToViewport(const TArray<FMovieSceneEntityID> InEntities)
	{
		Entities = InEntities;
	}

	void PreTask()
	{
		TransformToSet.Reset();
		if (GEditor)
		{
			for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
			{
				if (ViewportClient && 
						ViewportClient->IsPerspective() && 
						ViewportClient->GetViewMode() != VMI_Unknown &&
						ViewportClient->AllowsCinematicControl())
				{
					TransformToSet.Emplace(FTransform::Identity);
					TransformToSet->SetLocation(ViewportClient->GetViewLocation());
					TransformToSet->SetRotation(ViewportClient->GetViewRotation().Quaternion());
					break;
				}
			}
		}
	}
};

struct FSetInitialTransformToOrigin : FSetInitialTransformValues
{
	FSetInitialTransformToOrigin(const TArray<FMovieSceneEntityID> InEntities)
	{
		Entities = InEntities;
		TransformToSet = FTransform::Identity;
	}
};

}

bool UTemplateSequenceCameraPreviewSystem::bEnableNextFrame = false;
bool UTemplateSequenceCameraPreviewSystem::bDisableNextFrame = false;

void UTemplateSequenceCameraPreviewSystem::EnableNextFrame()
{
	bEnableNextFrame = true;
}

void UTemplateSequenceCameraPreviewSystem::DisableNextFrame()
{
	bDisableNextFrame = true;
}

UTemplateSequenceCameraPreviewSystem::UTemplateSequenceCameraPreviewSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Evaluation;
}

bool UTemplateSequenceCameraPreviewSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return bEnableNextFrame || bDisableNextFrame;
}

void UTemplateSequenceCameraPreviewSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	if (!(bEnableNextFrame || bDisableNextFrame))
	{
		return;
	}

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	// Find all the object/component binding IDs that relate to camera actors.
	TArray<FGuid> CameraBindingIDs;
	for (auto It = InstanceRegistry->GetSparseInstances().CreateConstIterator(); It; ++It)
	{
		const FSequenceInstance& Instance = (*It);
		if (!Instance.IsRootSequence())
		{
			continue;
		}

		TSharedRef<FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
		UTemplateSequence* RootSequence = Cast<UTemplateSequence>(SharedPlaybackState->GetRootSequence());
		if (!RootSequence)
		{
			continue;
		}

		const UObject* RootObjectTemplate = RootSequence->GetRootObjectSpawnableTemplate();
		if (!RootObjectTemplate || !RootObjectTemplate->IsA<ACameraActor>())
		{
			continue;
		}

		UMovieScene* MovieScene = RootSequence->GetMovieScene();
		if (!MovieScene || MovieScene->GetSpawnableCount() == 0)
		{
			continue;
		}

		const FMovieSceneSpawnable& FirstSpawnable = MovieScene->GetSpawnable(0);
		CameraBindingIDs.Add(FirstSpawnable.GetGuid());
	}

	// Now look for scene component binding entities that were imported from these object bindings.
	TArray<FMovieSceneEntityID> ParentsOfEntitiesToTag;

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->GenericObjectBinding)
	.FilterAll({ BuiltInComponents->Tags.ImportedEntity })
	.Iterate_PerEntity(&Linker->EntityManager,
			[&CameraBindingIDs, &ParentsOfEntitiesToTag](FMovieSceneEntityID EntityID, FGuid ObjectBindingID)
			{
				if (CameraBindingIDs.Contains(ObjectBindingID))
				{
					ParentsOfEntitiesToTag.Add(EntityID);
				}
			});

	// Next, look for transform entities generated from the ones we just found. We are looking for
	// the entities that hold the initial value of the camera transform. There are two situations:
	//
	// 1) They are non-blending entities that handle everything by themselves. Tag those with our
	//    custom tag.
	//
	// 2) They are part of a blending channel's inputs. In that case, we need to find the blending
	//	  channel output entity, because that's where the initial value is held.
	//
	TArray<FMovieSceneEntityID> EntitiesToTag;

	// Situation 1
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->ParentEntity)
	.FilterAll({
			TrackComponents->ComponentTransform.PropertyTag,
			TrackComponents->ComponentTransform.InitialValue })
	.FilterNone({ BuiltInComponents->BlendChannelInput })
	.Iterate_PerEntity(&Linker->EntityManager, 
			[&ParentsOfEntitiesToTag, &EntitiesToTag](
				FMovieSceneEntityID EntityID, FMovieSceneEntityID ParentEntityID)
			{
				if (ParentsOfEntitiesToTag.Contains(ParentEntityID))
				{
					EntitiesToTag.Add(EntityID);
				}
			});

	// Situation 2
	TArray<FMovieSceneBlendChannelID> BlendChannelsToTag;

	FEntityTaskBuilder()
	.Read(BuiltInComponents->ParentEntity)
	.Read(BuiltInComponents->BlendChannelInput)
	.Iterate_PerEntity(&Linker->EntityManager, 
			[&ParentsOfEntitiesToTag, &BlendChannelsToTag](
				FMovieSceneEntityID ParentEntityID, FMovieSceneBlendChannelID BlendChannelID)
			{
				if (ParentsOfEntitiesToTag.Contains(ParentEntityID))
				{
					BlendChannelsToTag.Add(BlendChannelID);
				}
			});

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->BlendChannelOutput)
	.FilterAll({ 
			TrackComponents->ComponentTransform.PropertyTag,
			TrackComponents->ComponentTransform.InitialValue })
	.Iterate_PerEntity(&Linker->EntityManager,
			[&BlendChannelsToTag, &EntitiesToTag](FMovieSceneEntityID EntityID, FMovieSceneBlendChannelID BlendChannelID)
			{
				 if (BlendChannelsToTag.Contains(BlendChannelID))
				 {
					EntitiesToTag.Add(EntityID);
				 }
			});

	if (bEnableNextFrame)
	{
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Write(TrackComponents->ComponentTransform.InitialValue)
		.FilterAll({ TrackComponents->ComponentTransform.PropertyTag })
		.Dispatch_PerAllocation<FSetInitialTransformToViewport>(
				&Linker->EntityManager, InPrerequisites, &Subsequents, EntitiesToTag);
	}
	else if (bDisableNextFrame)
	{
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Write(TrackComponents->ComponentTransform.InitialValue)
		.FilterAll({ TrackComponents->ComponentTransform.PropertyTag })
		.Dispatch_PerAllocation<FSetInitialTransformToOrigin>(
				&Linker->EntityManager, InPrerequisites, &Subsequents, EntitiesToTag);
	}

	if (!Linker->FindExtension<IInterrogationExtension>())
	{
		bEnableNextFrame = false;
		bDisableNextFrame = false;
	}
}

