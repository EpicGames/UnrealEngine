// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinkerSharedExtension.h"

#include "TemplateSequenceCameraPreviewSystem.generated.h"

/**
 * A sequencer system that can write the current editor viewport transform into the initial values
 * of the camera object's transform entities.
 */
UCLASS(MinimalAPI)
class UTemplateSequenceCameraPreviewSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UTemplateSequenceCameraPreviewSystem(const FObjectInitializer& ObjInit);

	/** Update the root camera's initial transform to the current editor viewport transform. */
	static void EnableNextFrame();
	/** Revert the root camera's initial transform to the origin. */
	static void DisableNextFrame();

protected:

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	static bool bEnableNextFrame;
	static bool bDisableNextFrame;
};

