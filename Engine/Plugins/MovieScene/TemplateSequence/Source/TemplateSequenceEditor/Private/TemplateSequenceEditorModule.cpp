// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraAnimationSequence.h"
#include "Customizations/CameraAnimationSequenceCustomization.h"
#include "Customizations/TemplateSequenceCustomization.h"
#include "Customizations/TemplateSequenceCustomizationBase.h"
#include "ISequencerModule.h"
#include "ISettingsModule.h"
#include "Misc/MovieSceneSequenceEditor_TemplateSequence.h"
#include "Misc/TemplateSequenceEditorSettings.h"
#include "Modules/ModuleManager.h"
#include "SequencerSettings.h"
#include "Styles/TemplateSequenceEditorStyle.h"
#include "TrackEditors/TemplateSequenceTrackEditor.h"

#define LOCTEXT_NAMESPACE "TemplateSequenceEditor"

/**
 * Implements the FTemplateSequenceEditor module.
 */
class FTemplateSequenceEditorModule : public IModuleInterface, public FGCObject
{
public:
	FTemplateSequenceEditorModule()
		: Settings(nullptr)
	{
	}

	virtual void StartupModule() override
	{
		RegisterSettings();
		RegisterSequenceEditor();
		RegisterTrackEditors();
		RegisterSequenceCustomizations();
	}

	virtual void ShutdownModule() override
	{
		UnregisterSequenceCustomizations();
		UnregisterTrackEditors();
		UnregisterSequenceEditor();
		UnregisterSettings();
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		if (Settings != nullptr)
		{
			Collector.AddReferencedObject(Settings);
		}
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTemplateSequenceEditorModule");
	}

private:
	void RegisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "TemplateSequenceEditor",
				LOCTEXT("TemplateSequenceEditorProjectSettingsName", "Template Sequence Editor"),
				LOCTEXT("TemplateSequenceEditorProjectSettingsDescription", "Configure the Template Sequence Editor."),
				GetMutableDefault<UTemplateSequenceEditorSettings>()
			);

			Settings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("TemplateSequenceEditor"));

			SettingsModule->RegisterSettings("Editor", "ContentEditors", "TemplateSequenceEditor",
				LOCTEXT("TemplateSequenceEditorSettingsName", "Template Sequence Editor"),
				LOCTEXT("TemplateSequenceEditorSettingsDescription", "Configure the look and feel of the Template Sequence Editor."),
				Settings);
		}
	}

	void UnregisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "TemplateSequenceEditor");
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "TemplateSequenceEditor");
		}
	}

	void RegisterSequenceEditor()
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		SequenceEditorHandle = SequencerModule.RegisterSequenceEditor(UTemplateSequence::StaticClass(), MakeUnique<FMovieSceneSequenceEditor_TemplateSequence>());
	}

	void UnregisterSequenceEditor()
	{
		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule)
		{
			SequencerModule->UnregisterSequenceEditor(SequenceEditorHandle);
		}
	}

	void RegisterTrackEditors()
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		TemplateSequenceTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FTemplateSequenceTrackEditor::CreateTrackEditor));
	}

	void UnregisterTrackEditors()
	{
		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule)
		{
			SequencerModule->UnRegisterTrackEditor(TemplateSequenceTrackCreateEditorHandle);
		}
	}

	void RegisterSequenceCustomizations()
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		SequencerModule.GetSequencerCustomizationManager()->RegisterInstancedSequencerCustomization(UTemplateSequence::StaticClass(),
				FOnGetSequencerCustomizationInstance::CreateLambda([]()
				{
					return new FTemplateSequenceCustomization();
				}));
		SequencerModule.GetSequencerCustomizationManager()->RegisterInstancedSequencerCustomization(UCameraAnimationSequence::StaticClass(),
				FOnGetSequencerCustomizationInstance::CreateLambda([]()
				{
					return new FCameraAnimationSequenceCustomization();
				}));
	}

	void UnregisterSequenceCustomizations()
	{
		if (UObjectInitialized())
		{
			ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
			SequencerModule.GetSequencerCustomizationManager()->UnregisterInstancedSequencerCustomization(UCameraAnimationSequence::StaticClass());
			SequencerModule.GetSequencerCustomizationManager()->UnregisterInstancedSequencerCustomization(UTemplateSequence::StaticClass());
		}
	}

private:

	FDelegateHandle SequenceEditorHandle;
	FDelegateHandle TemplateSequenceTrackCreateEditorHandle;

	TObjectPtr<USequencerSettings> Settings;
};

IMPLEMENT_MODULE(FTemplateSequenceEditorModule, TemplateSequenceEditor);

#undef LOCTEXT_NAMESPACE
