// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateSequenceEditorToolkit.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencerModule.h"
#include "LevelEditor.h"
#include "LevelEditorSequencerIntegration.h"
#include "Misc/TemplateSequenceEditorPlaybackContext.h"
#include "Misc/TemplateSequenceEditorSettings.h"
#include "Misc/TemplateSequenceEditorSpawnRegister.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Systems/TemplateSequenceCameraPreviewSystem.h"
#include "Widgets/Docking/SDockTab.h"
#include "TemplateSequence.h"

#define LOCTEXT_NAMESPACE "TemplateSequenceEditor"

const FName FTemplateSequenceEditorToolkit::SequencerMainTabId(TEXT("Sequencer_SequencerMain"));

namespace SequencerDefs
{
	static const FName SequencerAppIdentifier(TEXT("SequencerApp"));
}

FTemplateSequenceEditorToolkit::FTemplateSequenceEditorToolkit(const TSharedRef<ISlateStyle>& InStyle)
	: TemplateSequence(nullptr)
	, Style(InStyle)
{
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	int32 NewIndex = SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates().Add(
		FAssetEditorExtender::CreateRaw(this, &FTemplateSequenceEditorToolkit::HandleMenuExtensibilityGetExtender));
	SequencerExtenderHandle = SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates()[NewIndex].GetHandle();
}

FTemplateSequenceEditorToolkit::~FTemplateSequenceEditorToolkit()
{
	FLevelEditorSequencerIntegration::Get().RemoveSequencer(Sequencer.ToSharedRef());

	Sequencer->Close();

	if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		
		// @todo remove when world-centric mode is added
		LevelEditorModule.AttachSequencer(SNullWidget::NullWidget, nullptr);
		FLevelEditorSequencerIntegration::Get().RemoveSequencer(Sequencer.ToSharedRef());

		LevelEditorModule.OnMapChanged().RemoveAll(this);
	}

	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates().RemoveAll([this](const FAssetEditorExtender& Extender)
	{
		return SequencerExtenderHandle == Extender.GetHandle();
	});
}

void FTemplateSequenceEditorToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UTemplateSequence* InTemplateSequence, const FTemplateSequenceToolkitParams& ToolkitParams)
{
	// create tab layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_TemplateSequenceEditor")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->Split
			(
				FTabManager::NewStack()
				->AddTab(SequencerMainTabId, ETabState::OpenedTab)
			)
		);

	TemplateSequence = InTemplateSequence;
	PlaybackContext = MakeShared<FTemplateSequenceEditorPlaybackContext>();

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = false;

	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, SequencerDefs::SequencerAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, TemplateSequence);

	TSharedRef<FTemplateSequenceEditorSpawnRegister> SpawnRegister = MakeShareable(new FTemplateSequenceEditorSpawnRegister());

	// Initialize sequencer.
	FSequencerInitParams SequencerInitParams;
	{
		SequencerInitParams.RootSequence = TemplateSequence;
		SequencerInitParams.bEditWithinLevelEditor = true;
		SequencerInitParams.ToolkitHost = InitToolkitHost;
		SequencerInitParams.SpawnRegister = SpawnRegister;
		SequencerInitParams.HostCapabilities.bSupportsCurveEditor = true;
		SequencerInitParams.HostCapabilities.bSupportsSaveMovieSceneAsset = true;

		SequencerInitParams.PlaybackContext.Bind(PlaybackContext.ToSharedRef(), &FTemplateSequenceEditorPlaybackContext::GetPlaybackContext);

		SequencerInitParams.ViewParams.UniqueName = "TemplateSequenceEditor";
		SequencerInitParams.ViewParams.ScrubberStyle = ESequencerScrubberStyle::FrameBlock;
		SequencerInitParams.ViewParams.OnReceivedFocus.BindRaw(this, &FTemplateSequenceEditorToolkit::OnSequencerReceivedFocus);
	}

	Sequencer = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer").CreateSequencer(SequencerInitParams);
	SpawnRegister->SetSequencer(Sequencer);
	Sequencer->OnActorAddedToSequencer().AddSP(this, &FTemplateSequenceEditorToolkit::HandleActorAddedToSequencer);

	FLevelEditorSequencerIntegrationOptions Options;
	Options.bRequiresLevelEvents = true;
	Options.bRequiresActorEvents = true;
	FLevelEditorSequencerIntegration::Get().AddSequencer(Sequencer.ToSharedRef(), Options);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

	// Reopen the scene outliner so that is refreshed with the sequencer columns
	{
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		if (LevelEditorTabManager->FindExistingLiveTab(FName("LevelEditorSceneOutliner")).IsValid())
		{
			LevelEditorTabManager->TryInvokeTab(FName("LevelEditorSceneOutliner"))->RequestCloseTab();
			LevelEditorTabManager->TryInvokeTab(FName("LevelEditorSceneOutliner"));
		}
	}

	// If we are viewing a camera animation, see if we want to base it on the viewport transform from the start.
	const UTemplateSequenceEditorSettings* Settings = GetDefault<UTemplateSequenceEditorSettings>();
	if (Settings->bCameraInitiallyAdditiveToViewport)
	{
		UTemplateSequenceCameraPreviewSystem::EnableNextFrame();
	}

	TSharedPtr<SDockTab> DockTab = LevelEditorModule.AttachSequencer(Sequencer->GetSequencerWidget(), SharedThis(this));
	if (DockTab.IsValid())
	{
		TAttribute<FText> LabelSuffix = TAttribute<FText>(this, &FTemplateSequenceEditorToolkit::GetTabSuffix);
		DockTab->SetTabLabelSuffix(LabelSuffix);
	}

	LevelEditorModule.OnMapChanged().AddRaw(this, &FTemplateSequenceEditorToolkit::HandleMapChanged);
}

FText FTemplateSequenceEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Template Sequence Editor");
}

FName FTemplateSequenceEditorToolkit::GetToolkitFName() const
{
	static FName SequencerName("TemplateSequenceEditor");
	return SequencerName;
}

FString FTemplateSequenceEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Sequencer ").ToString();
}

FText FTemplateSequenceEditorToolkit::GetTabSuffix() const
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();

	if (Sequence == nullptr)
	{
		return FText::GetEmpty();
	}

	const bool bIsDirty = Sequence->GetMovieScene()->GetOuter()->GetOutermost()->IsDirty();
	if (bIsDirty)
	{
		return LOCTEXT("TabSuffixAsterix", "*");
	}

	return FText::GetEmpty();
}

FLinearColor FTemplateSequenceEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}

void FTemplateSequenceEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	if (IsWorldCentricAssetEditor())
	{
		return;
	}
}

void FTemplateSequenceEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	if (!IsWorldCentricAssetEditor())
	{
		InTabManager->UnregisterTabSpawner(SequencerMainTabId);
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.AttachSequencer(SNullWidget::NullWidget, nullptr);
}

TSharedRef<FExtender> FTemplateSequenceEditorToolkit::HandleMenuExtensibilityGetExtender(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> ContextSensitiveObjects)
{
	TSharedRef<FExtender> AddTrackMenuExtender(new FExtender());
	AddTrackMenuExtender->AddMenuExtension(
		SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection,
		EExtensionHook::Before,
		CommandList,
		FMenuExtensionDelegate::CreateRaw(this, &FTemplateSequenceEditorToolkit::HandleTrackMenuExtensionAddTrack, ContextSensitiveObjects));

	return AddTrackMenuExtender;
}

void FTemplateSequenceEditorToolkit::HandleTrackMenuExtensionAddTrack(FMenuBuilder& AddTrackMenuBuilder, TArray<UObject*> ContextObjects)
{
	// TODO-lchabant: stolen from level sequence.
	if (ContextObjects.Num() != 1)
	{
		return;
	}

	AActor* Actor = Cast<AActor>(ContextObjects[0]);
	if (Actor == nullptr)
	{
		return;
	}

	AddTrackMenuBuilder.BeginSection("Components", LOCTEXT("ComponentsSection", "Components"));
	{
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (Component)
			{
				FUIAction AddComponentAction(FExecuteAction::CreateSP(this, &FTemplateSequenceEditorToolkit::HandleAddComponentActionExecute, Component));
				FText AddComponentLabel = FText::FromString(Component->GetName());
				FText AddComponentToolTip = FText::Format(LOCTEXT("ComponentToolTipFormat", "Add {0} component"), FText::FromString(Component->GetName()));
				AddTrackMenuBuilder.AddMenuEntry(AddComponentLabel, AddComponentToolTip, FSlateIcon(), AddComponentAction);
			}
		}
	}
	AddTrackMenuBuilder.EndSection();
}

void FTemplateSequenceEditorToolkit::HandleAddComponentActionExecute(UActorComponent* Component)
{
	// TODO-lchabant: stolen from level sequence.
	const FScopedTransaction Transaction(LOCTEXT("AddComponent", "Add Component"));

	FString ComponentName = Component->GetName();

	TArray<UActorComponent*> ActorComponents;
	ActorComponents.Add(Component);

	USelection* SelectedActors = GEditor->GetSelectedActors();
	if (SelectedActors && SelectedActors->Num() > 0)
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = CastChecked<AActor>(*Iter);

			TArray<UActorComponent*> OutActorComponents;
			Actor->GetComponents(OutActorComponents);

			for (UActorComponent* ActorComponent : OutActorComponents)
			{
				if (ActorComponent->GetName() == ComponentName)
				{
					ActorComponents.AddUnique(ActorComponent);
				}
			}
		}
	}

	for (UActorComponent* ActorComponent : ActorComponents)
	{
		Sequencer->GetHandleToObject(ActorComponent);
	}
}

void FTemplateSequenceEditorToolkit::HandleActorAddedToSequencer(AActor* Actor, const FGuid Binding)
{
	// TODO-lchabant: add default tracks (re-use level sequence toolkit code).
}

void FTemplateSequenceEditorToolkit::HandleMapChanged(UWorld* NewWorld, EMapChangeType MapChangeType)
{
	if ((MapChangeType == EMapChangeType::LoadMap || MapChangeType == EMapChangeType::NewMap || MapChangeType == EMapChangeType::TearDownWorld))
	{
		Sequencer->GetSpawnRegister().CleanUp(*Sequencer);
		CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
	}
}

bool FTemplateSequenceEditorToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	return true;
}

bool FTemplateSequenceEditorToolkit::CanFindInContentBrowser() const
{
	// False so that sequencer doesn't take over Find In Content Browser functionality and always find the level sequence asset.
	return false;
}

void FTemplateSequenceEditorToolkit::OnSequencerReceivedFocus()
{
	if (Sequencer.IsValid())
	{
		FLevelEditorSequencerIntegration::Get().OnSequencerReceivedFocus(Sequencer.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE
