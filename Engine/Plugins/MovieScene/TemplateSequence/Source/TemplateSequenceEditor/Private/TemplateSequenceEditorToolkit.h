// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class UTemplateSequence;
enum class EMapChangeType : uint8;

class FToolBarBuilder;
class FTemplateSequenceEditorPlaybackContext;
class ISequencer;
class FAssetDragDropOp;
class FClassDragDropOp;
class FActorDragDropGraphEdOp;

struct FTemplateSequenceToolkitParams
{
	bool bCanChangeBinding = true;
};

/**
 * Implements an Editor toolkit for template sequences.
 */
class FTemplateSequenceEditorToolkit : public FAssetEditorToolkit, public FGCObject
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FTemplateSequenceEditorToolkit(const TSharedRef<ISlateStyle>& InStyle);

	/** Virtual destructor */
	virtual ~FTemplateSequenceEditorToolkit();

public:

	/**
	 * Initialize this asset editor.
	 *
	 * @param Mode Asset editing mode for this editor (standalone or world-centric).
	 * @param InitToolkitHost When Mode is WorldCentric, this is the level editor instance to spawn this editor within.
	 * @param TemplateSequence The animation to edit.
	 * @param TrackEditorDelegates Delegates to call to create auto-key handlers for this sequencer.
	 */
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UTemplateSequence* TemplateSequence, const FTemplateSequenceToolkitParams& ToolkitParams);

public:

	//~ FAssetEditorToolkit interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(TemplateSequence);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTemplateSequenceEditorToolkit");
	}

	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	virtual bool CanFindInContentBrowser() const override;

public:

	//~ IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FText GetTabSuffix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

private:

	TSharedRef<FExtender> HandleMenuExtensibilityGetExtender(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> ContextSensitiveObjects);
	void HandleTrackMenuExtensionAddTrack(FMenuBuilder& AddTrackMenuBuilder, TArray<UObject*> ContextObjects);
	void HandleAddComponentActionExecute(UActorComponent* Component);

	void HandleActorAddedToSequencer(AActor* Actor, const FGuid Binding);

	void HandleMapChanged(class UWorld* NewWorld, EMapChangeType MapChangeType);

	void OnSequencerReceivedFocus();

private:

	/** Template sequence for our edit operation. */
	TObjectPtr<UTemplateSequence> TemplateSequence;

	/** The sequencer used by this editor. */
	TSharedPtr<ISequencer> Sequencer;

	/** Pointer to the style set to use for toolkits. */
	TSharedRef<ISlateStyle> Style;

	/** Handle to the sequencer properties menu extender. */
	FDelegateHandle SequencerExtenderHandle;

	TSharedPtr<FTemplateSequenceEditorPlaybackContext> PlaybackContext;

	/**	The tab ids for all the tabs used */
	static const FName SequencerMainTabId;
};
