// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/TemplateSequenceTrackEditor.h"
#include "Camera/CameraComponent.h"
#include "Channels/MovieSceneChannelData.h"
#include "ContentBrowserModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageName.h"
#include "MVVM/Views/ViewUtilities.h"
#include "MovieScenePossessable.h"
#include "TemplateSequence.h"
#include "MovieSceneSpawnable.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "ScopedTransaction.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/TemplateSequenceSection.h"
#include "SequencerSettings.h"
#include "Tracks/TemplateSequenceTrack.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FTemplateSequenceTrackEditor"

FTemplateSequenceTrackEditor::FTemplateSequenceTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
}

TSharedRef<ISequencerTrackEditor> FTemplateSequenceTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FTemplateSequenceTrackEditor(InSequencer));
}

FText FTemplateSequenceTrackEditor::GetDisplayName() const
{
	return LOCTEXT("TemplateSequenceTrackEditor_DisplayName", "Template Sequence");
}

bool FTemplateSequenceTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UTemplateSequenceTrack::StaticClass();
}

bool FTemplateSequenceTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UTemplateSequenceTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && (InSequence->IsA(ULevelSequence::StaticClass()) || InSequence->IsA(UTemplateSequence::StaticClass()))) || TrackSupported == ETrackSupport::Supported;
}

void FTemplateSequenceTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(AActor::StaticClass()))
	{
		const FText SubMenuEntryText = LOCTEXT("AddTemplateSequence", "Template Sequence");
		const FText SubMenuEntryTooltip = LOCTEXT("AddTemplateSequenceTooltip", "Adds a track that can play a template sequence asset using the parent binding.");
		
		MenuBuilder.AddSubMenu(
			SubMenuEntryText, SubMenuEntryTooltip,
			FNewMenuDelegate::CreateRaw(this, &FTemplateSequenceTrackEditor::AddTemplateSequenceAssetSubMenu, ObjectBindings, ObjectClass)
		);
	}
}

TSharedPtr<SWidget> FTemplateSequenceTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	const UClass* ObjectClass = AcquireObjectClassFromObjectGuid(ObjectBinding);

	if (ObjectClass != nullptr)
	{
		return UE::Sequencer::MakeAddButton(
					LOCTEXT("TemplateSequenceAddButton", "Template Sequence"),
					FOnGetContent::CreateSP(this, &FTemplateSequenceTrackEditor::BuildTemplateSequenceAssetSubMenu, ObjectBinding, ObjectClass),
					Params.ViewModel);
	}
	else
	{
		return TSharedPtr<SWidget>();
	}
}

TSharedRef<ISequencerSection> FTemplateSequenceTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );
	
	return MakeShareable(new FTemplateSequenceSection(GetSequencer(), *CastChecked<UTemplateSequenceSection>(&SectionObject)));
}

class STemplateSequenceAssetSubMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STemplateSequenceAssetSubMenu)
	{}
	SLATE_END_ARGS();

	void Construct(const FArguments& Args, TSharedPtr<FTemplateSequenceTrackEditor> InTrackEditor, TArray<FGuid> ObjectBindings, const UClass* InBaseClass)
	{
		check(InBaseClass != nullptr);
		BaseClass = InBaseClass;

		TrackEditor = InTrackEditor;

		GConfig->GetBool(TEXT("TemplateSequence"), TEXT("ShowOutdatedAssetsInCameraAnimationTrackEditor"), bShowingHiddenAssets, GEditorPerProjectIni);

		// We will be looking for any template sequence whose root bound object is compatible with the current object binding.
		// That means any template sequence bound to something of the same class, or a parent class.
		TSet<FString> BaseClassNames;
		const UClass* CurBaseClass = InBaseClass;
		while (CurBaseClass)
		{
			BaseClassNames.Add(CurBaseClass->GetPathName());
			CurBaseClass = CurBaseClass->GetSuperClass();
		}

		TSharedRef<FTemplateSequenceTrackEditor> TrackEditorRef = TrackEditor.ToSharedRef();
		TSharedPtr<ISequencer> Sequencer = TrackEditorRef->GetSequencer();

		FAssetPickerConfig AssetPickerConfig;
		{
			UMovieSceneSequence* Sequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;

			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(TrackEditorRef, &FTemplateSequenceTrackEditor::OnTemplateSequenceAssetSelected, ObjectBindings);
			AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(TrackEditorRef, &FTemplateSequenceTrackEditor::OnTemplateSequenceAssetEnterPressed, ObjectBindings);
			AssetPickerConfig.RefreshAssetViewDelegates.Add(&RefreshAssetViewDelegate);
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bAddFilterUI = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.Filter.ClassPaths.Add(UTemplateSequence::StaticClass()->GetClassPathName());
			AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
			AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));

			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda(
				[this, BaseClassNames](const FAssetData& AssetData) -> bool
				{
					FString FoundBoundActorClass;
					const FAssetDataTagMapSharedView::FFindTagResult FoundBoundActorClassTag = AssetData.TagsAndValues.FindTag("BoundActorClass");
					if (FoundBoundActorClassTag.IsSet())
					{
						FString TagValue = FoundBoundActorClassTag.GetValue();
						if (FPackageName::IsShortPackageName(TagValue))
						{
							// Replace the short name with the full path name if possible.
							FTopLevelAssetPath BoundActorClassPath = UClass::TryConvertShortTypeNameToPathName<UStruct>(TagValue, ELogVerbosity::Warning, TEXT("STemplateSequenceAssetSubMenu"));
							if (!BoundActorClassPath.IsNull())
							{
								FoundBoundActorClass = BoundActorClassPath.ToString();
							}
						}
						else
						{
							FoundBoundActorClass = TagValue;
						}
					}
					if (!FoundBoundActorClass.IsEmpty())
					{
						// Filter this out if it's got an incompatible bound actor class.
						return !BaseClassNames.Contains(FoundBoundActorClass);
					}
					else
					{
						// Old asset, hasn't been saved since we added the bound actor class in the tags.
						++NumAssetsRequiringSave;
						// Don't filter if we're showing old assets, do filter if we only want compatible assets.
						return bShowingHiddenAssets ? false : true;
					}
				});
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		AssetPicker = ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);

		const float WidthOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = Sequencer.IsValid() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(WidthOverride)
			.HeightOverride(HeightOverride)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					AssetPicker.ToSharedRef()
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(EVerticalAlignment::VAlign_Center)
					.FillWidth(1.f)
					[
						SNew(STextBlock)
						.Margin(FMargin(5.f))
						.TextStyle(FAppStyle::Get(), "SmallText.Subdued")
						.Visibility(this, &STemplateSequenceAssetSubMenu::GetBottomRowVisibility)
						.Text_Lambda([this]()
							{
								return !bShowingHiddenAssets ? 
									FText::Format(LOCTEXT("OutdatedAssetsWarning", "Hiding {0} outdated assets"), FText::AsNumber(NumAssetsRequiringSave)) :
									LOCTEXT("OutdatedAssetsMessage", "Showing outdated assets");
							})
					]
					+SHorizontalBox::Slot()
					.VAlign(EVerticalAlignment::VAlign_Center)
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.TextStyle(FAppStyle::Get(), "SmallText")
						.Visibility(this, &STemplateSequenceAssetSubMenu::GetBottomRowVisibility)
						.Text_Lambda([this]() { return !bShowingHiddenAssets ? 
								LOCTEXT("ShowOutdatedAssetsButton", "Show Outdated Assets") : 
								LOCTEXT("HideOutdatedAssetsButton", "Hide Outdated Assets"); })
						.ToolTipText(
							LOCTEXT("ShowAssetsRequiringSaveButtonTooltip", "Show or hide legacy assets (like CameraAnim) and template sequences that are potentially incompatible with the currently selected object binding (re-save those to fix the issue).")
							)
						.OnClicked(this, &STemplateSequenceAssetSubMenu::OnShowHiddenAssets)
					]
				]
			]
		];
	}

	EVisibility GetBottomRowVisibility() const
	{
		return (NumAssetsRequiringSave) > 0 ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FReply OnShowHiddenAssets()
	{
		bShowingHiddenAssets = !bShowingHiddenAssets;
		GConfig->SetBool(TEXT("TemplateSequence"), TEXT("ShowOutdatedAssetsInCameraAnimationTrackEditor"), bShowingHiddenAssets, GEditorPerProjectIni);

		NumAssetsRequiringSave = 0;
		RefreshAssetViewDelegate.ExecuteIfBound(true);
		return FReply::Handled();
	}

private:
	TSharedPtr<FTemplateSequenceTrackEditor> TrackEditor;
	const UClass* BaseClass;
	const UClass* LegacyBaseClass;

	TSharedPtr<SWidget> AssetPicker;
	uint32 NumAssetsRequiringSave = 0;
	bool bShowingHiddenAssets = false;
	FRefreshAssetViewDelegate RefreshAssetViewDelegate;
};

void FTemplateSequenceTrackEditor::AddTemplateSequenceAssetSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, const UClass* RootBindingClass)
{
	if (ObjectBindings.Num() > 0 && RootBindingClass != nullptr)
	{
		TSharedPtr<STemplateSequenceAssetSubMenu> MenuEntry = SNew(STemplateSequenceAssetSubMenu, SharedThis(this), ObjectBindings, RootBindingClass);
		MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
	}
}

TSharedRef<SWidget> FTemplateSequenceTrackEditor::BuildTemplateSequenceAssetSubMenu(FGuid ObjectBinding, const UClass* RootBindingClass)
{
	check(RootBindingClass != nullptr);
	
	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);
	return SNew(STemplateSequenceAssetSubMenu, SharedThis(this), ObjectBindings, RootBindingClass);
}

void FTemplateSequenceTrackEditor::OnTemplateSequenceAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings)
{
	FSlateApplication::Get().DismissAllMenus();

	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();
	if (FocusedMovieScene == nullptr || FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	if (UTemplateSequence* SelectedSequence = Cast<UTemplateSequence>(AssetData.GetAsset()))
	{
		const FScopedTransaction Transaction(LOCTEXT("AddTemplateSequence_Transaction", "Add Template Animation"));

		AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FTemplateSequenceTrackEditor::AddKeyInternal, ObjectBindings, SelectedSequence));
	}
}

void FTemplateSequenceTrackEditor::OnTemplateSequenceAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings)
{
	if (AssetData.Num() > 0)
	{
		OnTemplateSequenceAssetSelected(AssetData[0].GetAsset(), ObjectBindings);
	}
}

FKeyPropertyResult FTemplateSequenceTrackEditor::AddKeyInternal(FFrameNumber KeyTime, FGuid ObjectBinding, UTemplateSequence* TemplateSequence)
{
	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);
	return AddKeyInternal(KeyTime, ObjectBindings, TemplateSequence);
}

FKeyPropertyResult FTemplateSequenceTrackEditor::AddKeyInternal(FFrameNumber KeyTime, TArray<FGuid> ObjectBindings, UTemplateSequence* TemplateSequence)
{
	FKeyPropertyResult KeyPropertyResult;

	if (TemplateSequence->GetMovieScene()->GetPlaybackRange().IsEmpty())
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("InvalidSequenceDuration", "Invalid template sequence {0}. The template sequence has no duration."), TemplateSequence->GetDisplayName()));
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info);
		return KeyPropertyResult;
	}

	if (!CanAddSubSequence(*TemplateSequence))
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("InvalidSequenceCycle", "Invalid template sequence {0}. There could be a circular dependency."), TemplateSequence->GetDisplayName()));
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info);
		return KeyPropertyResult;
	}

	TArray<UMovieSceneSection*> NewSections;
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		const FFrameRate OuterTickResolution = SequencerPtr->GetFocusedTickResolution();
		const FFrameRate SubTickResolution = TemplateSequence->GetMovieScene()->GetTickResolution();
		if (SubTickResolution != OuterTickResolution)
		{
			FNotificationInfo Info(FText::Format(LOCTEXT("TickResolutionMismatch", "The parent sequence has a different tick resolution {0} than the newly added sequence {1}"), OuterTickResolution.ToPrettyText(), SubTickResolution.ToPrettyText()));
			Info.bUseLargeFont = false;
			FSlateNotificationManager::Get().AddNotification(Info);
		}

		for (const FGuid& ObjectBindingGuid : ObjectBindings)
		{
			if (ObjectBindingGuid.IsValid())
			{
				FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectBindingGuid, UTemplateSequenceTrack::StaticClass());
				UMovieSceneTrack* Track = TrackResult.Track;
				KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

				if (ensure(Track) && Track->CanModify())
				{
					Track->Modify();

					UMovieSceneSection* NewSection = Cast<UTemplateSequenceTrack>(Track)->AddNewTemplateSequenceSection(KeyTime, TemplateSequence);
					KeyPropertyResult.bTrackModified = true;
					KeyPropertyResult.SectionsCreated.Add(NewSection);
					NewSections.Add(NewSection);
				}
			}
		}
	}

	if (NewSections.Num() > 0)
	{
		GetSequencer()->EmptySelection();
		for (UMovieSceneSection* NewSection : NewSections)
		{
			GetSequencer()->SelectSection(NewSection);
		}
		GetSequencer()->ThrobSectionSelection();
	}

	return KeyPropertyResult;
}

bool FTemplateSequenceTrackEditor::CanAddSubSequence(const UMovieSceneSequence& Sequence) const
{
	UMovieSceneSequence* FocusedSequence = GetSequencer()->GetFocusedMovieSceneSequence();
	return FSubTrackEditorUtil::CanAddSubSequence(FocusedSequence, Sequence);
}

const UClass* FTemplateSequenceTrackEditor::AcquireObjectClassFromObjectGuid(const FGuid& Guid)
{
	if (!Guid.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid() || SequencerPtr->GetFocusedMovieSceneSequence() == nullptr ||
			SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene() == nullptr)
	{
		return nullptr;
	}

	UMovieScene* FocusedMovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
	const FMovieSceneBinding* ObjectBinding = FocusedMovieScene->FindBinding(Guid);
	if (ObjectBinding == nullptr)
	{
		return nullptr;
	}

	const UClass* ObjectClass = nullptr;
	if (FMovieScenePossessable* Possessable = FocusedMovieScene->FindPossessable(Guid))
	{
		ObjectClass = Possessable->GetPossessedObjectClass();
	}
	else if (FMovieSceneSpawnable* Spawnable = FocusedMovieScene->FindSpawnable(Guid))
	{
		if (Spawnable->GetObjectTemplate() != nullptr)
		{
			ObjectClass = Spawnable->GetObjectTemplate()->GetClass();
		}
	}
	return ObjectClass;
}

UCameraComponent* FTemplateSequenceTrackEditor::AcquireCameraComponentFromObjectGuid(const FGuid& Guid)
{
	USkeleton* Skeleton = nullptr;
	for (TWeakObjectPtr<> WeakObject : GetSequencer()->FindObjectsInCurrentSequence(Guid))
	{
		UObject* const Obj = WeakObject.Get();
	
		if (AActor* const Actor = Cast<AActor>(Obj))
		{
			UCameraComponent* const CameraComp = MovieSceneHelpers::CameraComponentFromActor(Actor);
			if (CameraComp)
			{
				return CameraComp;
			}
		}
		else if (UCameraComponent* const CameraComp = Cast<UCameraComponent>(Obj))
		{
			if (CameraComp->IsActive())
			{
				return CameraComp;
			}
		}
	}

	return nullptr;
}

bool FTemplateSequenceTrackEditor::IsResizable(UMovieSceneTrack* InTrack) const
{
	return true;
}

void FTemplateSequenceTrackEditor::Resize(float NewSize, UMovieSceneTrack* InTrack)
{
	UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(InTrack);
	if (SubTrack)
	{
		SubTrack->Modify();

		int32 MaxNumRows = 1;
		for (UMovieSceneSection* Section : SubTrack->GetAllSections())
		{
			MaxNumRows = FMath::Max(MaxNumRows, Section->GetRowIndex() + 1);
		}

		SubTrack->SetRowHeight(FMath::RoundToInt(NewSize) / MaxNumRows);
	}
}

FTemplateSequenceSection::FTemplateSequenceSection(TSharedPtr<ISequencer> InSequencer, UTemplateSequenceSection& InSection)
	: TSubSectionMixin(InSequencer, InSection)
{
}

bool FTemplateSequenceSection::RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath)
{
	bool bAnyDeleted = false;
	FScopedTransaction DeletePropertyMultiplierTransaction(LOCTEXT("DeletePropertyMultiplier_Transaction", "Delete Property Multiplier"));
 	
	TArray<FScalablePropertyInfo> AllAnimatedProperties = GetAnimatedProperties();

	for (const FScalablePropertyInfo& AnimatedProperty : AllAnimatedProperties)
	{
		const FMovieScenePropertyBinding& SubPropertyBinding = AnimatedProperty.Get<1>();
		FText ScaleTypeSuffix = GetAnimatedPropertyScaleTypeSuffix(AnimatedProperty);

		FString KeyAreaName = SubPropertyBinding.PropertyPath.ToString() + ScaleTypeSuffix.ToString();
		if (KeyAreaNamePath.Contains(FName(*KeyAreaName)))
		{
			const int32 AlreadyAddedPropertyIndex = GetPropertyScaleFor(AnimatedProperty);
			if (AlreadyAddedPropertyIndex != INDEX_NONE)
			{
				UTemplateSequenceSection& TemplateSequenceSection = GetSectionObjectAs<UTemplateSequenceSection>();
				TemplateSequenceSection.Modify();
				TemplateSequenceSection.RemovePropertyScale(AlreadyAddedPropertyIndex);
				bAnyDeleted = true;
			}
		}
	}
	return bAnyDeleted;
}

void FTemplateSequenceSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
{
	TSubSectionMixin::BuildSectionContextMenu(MenuBuilder, ObjectBinding);

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddPropertyScalingMenu", "Property Multipliers"),
		LOCTEXT("AddPropertyScalingMenuTooltip", "Adds a multiplier channel that affects an animated property of the child sequence"),
		FNewMenuDelegate::CreateRaw(this, &FTemplateSequenceSection::BuildPropertyScalingSubMenu, ObjectBinding),
		false /*bInOpenSubMenuOnClick*/,
		FSlateIcon()//"LevelSequenceEditorStyle", "LevelSequenceEditor.PossessNewActor")
	);
}

void FTemplateSequenceSection::BuildPropertyScalingSubMenu(FMenuBuilder& MenuBuilder, FGuid ObjectBinding)
{
	// Get the list of all animated properties in the child template sequence. This is the list of properties for which
	// we can add a property multiplier.
	TArray<FScalablePropertyInfo> AllAnimatedProperties = GetAnimatedProperties();

	if (AllAnimatedProperties.Num() > 0)
	{
		for (const FScalablePropertyInfo& AnimatedProperty : AllAnimatedProperties)
		{
			const FMovieScenePropertyBinding& SubPropertyBinding = AnimatedProperty.Get<1>();
			FText ScaleTypeSuffix = GetAnimatedPropertyScaleTypeSuffix(AnimatedProperty);

			// We need to explicitly capture a non-const version of "this" otherwise the c++ compiler gets confused about 
			// which "GetSectionObjectAs<T>" to call inside of the lambda below.
			FTemplateSequenceSection* NonConstThis = this;

			FUIAction MenuEntryAction = FUIAction(
					FExecuteAction::CreateLambda(
						[NonConstThis, AnimatedProperty]()
						{
							FScopedTransaction TogglePropertyMultiplierTransaction(LOCTEXT("TogglePropertyMultiplier_Transaction", "Toggle Property Multiplier"));

							TSharedPtr<ISequencer> Sequencer = NonConstThis->GetSequencer();
							UTemplateSequenceSection& TemplateSequenceSection = NonConstThis->GetSectionObjectAs<UTemplateSequenceSection>();

							TemplateSequenceSection.Modify();

							const int32 AlreadyAddedPropertyIndex = NonConstThis->GetPropertyScaleFor(AnimatedProperty);
							if (AlreadyAddedPropertyIndex == INDEX_NONE)
							{
								FTemplateSectionPropertyScale NewPropertyScale;
								NewPropertyScale.ObjectBinding = AnimatedProperty.Get<0>();
								NewPropertyScale.PropertyBinding = AnimatedProperty.Get<1>();
								NewPropertyScale.PropertyScaleType = AnimatedProperty.Get<2>();
								NewPropertyScale.FloatChannel.SetDefault(1.f);

								TemplateSequenceSection.AddPropertyScale(NewPropertyScale);
							}
							else
							{
								TemplateSequenceSection.RemovePropertyScale(AlreadyAddedPropertyIndex);
							}

							Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
						}),
					FCanExecuteAction::CreateLambda([]() { return true; }),
					FGetActionCheckState::CreateLambda(
						[this, AnimatedProperty]() -> ECheckBoxState
						{
							const int32 AlreadyAddedPropertyIndex = GetPropertyScaleFor(AnimatedProperty);
							return AlreadyAddedPropertyIndex != INDEX_NONE ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					);

			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("AddPropertyScaling", "{0}{1}"), FText::FromName(SubPropertyBinding.PropertyPath), ScaleTypeSuffix),
				FText::Format(LOCTEXT("AddPropertyScalingTooltip", "Toggle the multiplier channel for property: {0}"),
					FText::FromName(SubPropertyBinding.PropertyPath)),
				FSlateIcon(),
				MenuEntryAction,
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}
	}
	else
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NoAvailablePropertyScaling", "None"), 
			LOCTEXT("NoAvailablePropertyScalingTooltip", "No animated properties are available for scaling"),
			FSlateIcon(),
			FUIAction(),
			NAME_None,
			EUserInterfaceActionType::None);
	}
}

TArray<FTemplateSequenceSection::FScalablePropertyInfo> FTemplateSequenceSection::GetAnimatedProperties() const
{
	TArray<FScalablePropertyInfo> AllAnimatedProperties;

	const UTemplateSequenceSection& TemplateSequenceSection = GetSectionObjectAs<UTemplateSequenceSection>();
	UMovieSceneSequence* SubSequence = TemplateSequenceSection.GetSequence();
	if (!SubSequence)
	{
		return AllAnimatedProperties;
	}

	const UMovieScene* SubMovieScene = SubSequence->GetMovieScene();
	if (!SubMovieScene)
	{
		return AllAnimatedProperties;
	}

	for (const FMovieSceneBinding& Binding : SubMovieScene->GetBindings())
	{
		for (const UMovieSceneTrack* BindingTrack : Binding.GetTracks())
		{
			if (const UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(BindingTrack))
			{
				const UClass* PropertyTrackClass = PropertyTrack->GetClass();
				TArray<ETemplateSectionPropertyScaleType, TInlineAllocator<2>> SupportedScaleTypes;
				if (PropertyTrackClass->IsChildOf<UMovieSceneFloatTrack>())
				{
					SupportedScaleTypes.Add(ETemplateSectionPropertyScaleType::FloatProperty);
				}
				else if (PropertyTrackClass->IsChildOf<UMovieScene3DTransformTrack>())
				{
					SupportedScaleTypes.Add(ETemplateSectionPropertyScaleType::TransformPropertyLocationOnly);
					SupportedScaleTypes.Add(ETemplateSectionPropertyScaleType::TransformPropertyRotationOnly);
				}

				for (ETemplateSectionPropertyScaleType ScaleType : SupportedScaleTypes)
				{
					AllAnimatedProperties.Add(FScalablePropertyInfo(
							Binding.GetObjectGuid(), PropertyTrack->GetPropertyBinding(), ScaleType));
				}
			}
		}
	}

	return AllAnimatedProperties;
}

int32 FTemplateSequenceSection::GetPropertyScaleFor(const FScalablePropertyInfo& AnimatedProperty) const
{
	const FGuid& SubObjectBinding = AnimatedProperty.Get<0>();
	const FMovieScenePropertyBinding& SubPropertyBinding = AnimatedProperty.Get<1>();
	const ETemplateSectionPropertyScaleType ScaleType = AnimatedProperty.Get<2>();

	const UTemplateSequenceSection& TemplateSequenceSection = GetSectionObjectAs<UTemplateSequenceSection>();

	for (int32 Index = 0; Index < TemplateSequenceSection.PropertyScales.Num(); ++Index)
	{
		const FTemplateSectionPropertyScale& PropertyScale(TemplateSequenceSection.PropertyScales[Index]);
		if (PropertyScale.ObjectBinding == SubObjectBinding &&
				PropertyScale.PropertyBinding == SubPropertyBinding &&
				PropertyScale.PropertyScaleType == ScaleType)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

FText FTemplateSequenceSection::GetAnimatedPropertyScaleTypeSuffix(const FScalablePropertyInfo& AnimatedProperty) const
{
	const ETemplateSectionPropertyScaleType ScaleType = AnimatedProperty.Get<2>();

	FText ScaleTypeSuffix;
	switch (ScaleType)
	{
		case ETemplateSectionPropertyScaleType::TransformPropertyLocationOnly:
			ScaleTypeSuffix = LOCTEXT("TransformPropertyLocationOnlyScaleSuffix", "[Location]");
			break;
		case ETemplateSectionPropertyScaleType::TransformPropertyRotationOnly:
			ScaleTypeSuffix = LOCTEXT("TransformPropertyRotationOnlyScaleSuffix", "[Rotation]");
			break;
		default:
			// Leave empty
			break;
	}

	return ScaleTypeSuffix;
}

#undef LOCTEXT_NAMESPACE
