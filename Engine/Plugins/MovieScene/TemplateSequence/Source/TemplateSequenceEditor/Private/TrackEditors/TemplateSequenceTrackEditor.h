// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrackEditor.h"
#include "TrackEditors/SubTrackEditorBase.h"

class UTemplateSequenceSection;
enum class ETemplateSectionPropertyScaleType;
struct FBuildEditWidgetParams;
struct FMovieScenePropertyBinding;

struct FAssetData;
class FMenuBuilder;
class UCameraComponent;
class UTemplateSequence;

class FTemplateSequenceTrackEditor : public FMovieSceneTrackEditor
{
public:
	FTemplateSequenceTrackEditor(TSharedRef<ISequencer> InSequencer);

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:
	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual bool SupportsType(TSubclassOf<class UMovieSceneTrack> TrackClass) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool IsResizable(UMovieSceneTrack* InTrack) const override;
	virtual void Resize(float NewSize, UMovieSceneTrack* InTrack) override;

private:
	void AddTemplateSequenceAssetSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, const UClass* RootBindingClass);
	TSharedRef<SWidget> BuildTemplateSequenceAssetSubMenu(FGuid ObjectBinding, const UClass* RootBindingClass);

	void OnTemplateSequenceAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings);
	void OnTemplateSequenceAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings);

	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, FGuid ObjectBinding, UTemplateSequence* TemplateSequence);
	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, TArray<FGuid> ObjectBindings, UTemplateSequence* TemplateSequence);

	bool CanAddSubSequence(const UMovieSceneSequence& Sequence) const;
	const UClass* AcquireObjectClassFromObjectGuid(const FGuid& Guid);
	UCameraComponent* AcquireCameraComponentFromObjectGuid(const FGuid& Guid);

	friend class STemplateSequenceAssetSubMenu;
};

class FTemplateSequenceSection
	: public TSubSectionMixin<>
	, public TSharedFromThis<FTemplateSequenceSection>
{
public:
	FTemplateSequenceSection(TSharedPtr<ISequencer> InSequencer, UTemplateSequenceSection& InSection);
	virtual ~FTemplateSequenceSection() = default;

	virtual bool RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath) override;
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) override;

private:

	void BuildPropertyScalingSubMenu(FMenuBuilder& MenuBuilder, FGuid ObjectBinding);
	
	typedef TTuple<FGuid, FMovieScenePropertyBinding, ETemplateSectionPropertyScaleType> FScalablePropertyInfo;
	TArray<FScalablePropertyInfo> GetAnimatedProperties() const;
	int32 GetPropertyScaleFor(const FScalablePropertyInfo& AnimatedProperty) const;
	FText GetAnimatedPropertyScaleTypeSuffix(const FScalablePropertyInfo& AnimatedProperty) const;
};

