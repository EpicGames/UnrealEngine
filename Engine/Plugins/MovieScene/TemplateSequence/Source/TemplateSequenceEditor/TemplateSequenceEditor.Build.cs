// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TemplateSequenceEditor : ModuleRules
{
	public TemplateSequenceEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"TemplateSequence",
				"BlueprintGraph",
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"InputCore",
				"Kismet",
				"MovieScene",
				"MovieSceneTools",
				"MovieSceneTracks",
				"Sequencer",
				"SequencerCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"TimeManagement",
				"LevelSequence",
				"LevelSequenceEditor"
			}
		);

		var DynamicModuleNames = new string[] {
			"LevelEditor",
			"PropertyEditor",
		};

		foreach (var Name in DynamicModuleNames)
		{
			PrivateIncludePathModuleNames.Add(Name);
			DynamicallyLoadedModuleNames.Add(Name);
		}
	}
}
