{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Template Sequence",
	"Description" : "Runtime for template sequences",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "TemplateSequence",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "TemplateSequenceEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}
